{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "GeoReferencing",
	"Description": "GeoReferencing tools for UE worlds",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "GeoReferencing",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Android",
				"IOS",
				"Linux",
				"Mac",
				"Win64"
			]
		},
		{
			"Name": "GeoReferencingEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Android",
				"IOS",
				"Linux",
				"Mac",
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "SQLiteCore",
			"Enabled": true
		}
	]
}