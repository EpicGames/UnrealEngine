// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

using System;
using System.IO;
using System.Collections.Generic;

public class GeoReferencing : ModuleRules
{
	public GeoReferencing(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"InputCore",
				"Engine",
				"PROJ",
				"Projects",
				"SQLiteCore", 
				"RHI"
			}
		);

		if (Target.bCompileAgainstEditor)
		{
			PrivateDependencyModuleNames.Add("Slate");
		}

		string ProjRedistFolder = Path.Combine(PluginDirectory, @"Resources/PROJ/*");
		RuntimeDependencies.Add(ProjRedistFolder, StagedFileType.UFS);
	}
}
