// Copyright Epic Games, Inc. All Rights Reserved.

#include "Ellipsoid.h"
#include "GeographicCoordinates.h"
#include "MathUtil.h"


// LWC_TODO - To be replaced once FVector::Normalize will use a smaller number than 1e-8
#define GEOREF_DOUBLE_SMALL_NUMBER (1.e-50)


FEllipsoid::FEllipsoid()
	: FEllipsoid(1.0, 1.0, 1.0)
{

}

FEllipsoid::FEllipsoid(double RadiusX, double RadiusY, double RadiusZ)
	: FEllipsoid(FVector(RadiusX, RadiusY, RadiusZ))
{

}

FEllipsoid::FEllipsoid(const FVector& InRadii)
	: Radii(InRadii)
	, RadiiSquared(InRadii.X * InRadii.X, InRadii.Y * InRadii.Y, InRadii.Z * InRadii.Z)
{
	check(InRadii.X != 0 && InRadii.Y != 0 && InRadii.Z != 0);

	OneOverRadii = FVector(1.0 / InRadii.X, 1.0 / InRadii.Y, 1.0 / InRadii.Z);
	OneOverRadiiSquared = FVector(1.0 / (InRadii.X * InRadii.X), 1.0 / (InRadii.Y * InRadii.Y), 1.0 / (InRadii.Z * InRadii.Z));
}

double FEllipsoid::GetMaximumRadius()
{
	return FMathd::Max3(Radii.X,Radii.Y, Radii.Z);
}

double FEllipsoid::GetMinimumRadius()
{
	return FMathd::Min3(Radii.X, Radii.Y, Radii.Z);
}

FVector FEllipsoid::GeodeticSurfaceNormal(const FVector& ECEFLocation) const
{
	FVector Normal( ECEFLocation.X * OneOverRadiiSquared.X, ECEFLocation.Y * OneOverRadiiSquared.Y, ECEFLocation.Z * OneOverRadiiSquared.Z);
	Normal.Normalize(GEOREF_DOUBLE_SMALL_NUMBER); 
	return Normal;
}

FVector FEllipsoid::GeodeticSurfaceNormal(const FGeographicCoordinates& GeographicCoordinates) const
{
	double LongitudeRad = FMathd::DegToRad * GeographicCoordinates.Longitude ;
	double LatitudeRad = FMathd::DegToRad * GeographicCoordinates.Latitude;
	double cosLatitude = FMathd::Cos(LatitudeRad);

	FVector Normal(cosLatitude * FMathd::Cos(LongitudeRad), cosLatitude * FMathd::Sin(LongitudeRad), FMathd::Sin(LatitudeRad));
	Normal.Normalize(GEOREF_DOUBLE_SMALL_NUMBER);
	return Normal;
}
