// Copyright Epic Games, Inc. All Rights Reserved.


#include "GeoReferencingBFL.h"

FText UGeoReferencingBFL::ToFullText(FVector& CartesianCoordinates, int32 IntegralDigits /*= 3*/)
{
	FNumberFormattingOptions NumberFormatOptions = FNumberFormattingOptions::DefaultNoGrouping();
	NumberFormatOptions.MinimumFractionalDigits = IntegralDigits;
	NumberFormatOptions.MaximumFractionalDigits = IntegralDigits;

	FFormatNamedArguments Args;
	Args.Add(TEXT("X"), FText::AsNumber(CartesianCoordinates.X, &NumberFormatOptions));
	Args.Add(TEXT("Y"), FText::AsNumber(CartesianCoordinates.Y, &NumberFormatOptions));
	Args.Add(TEXT("Z"), FText::AsNumber(CartesianCoordinates.Z, &NumberFormatOptions));

	return FText::Format(NSLOCTEXT("GeoReferencing", "CartesianCoordinatesFull", "Easting={X} Northing={Y} Up={Z}"), Args);
}

FText UGeoReferencingBFL::ToCompactText(FVector& CartesianCoordinates, int32 IntegralDigits /*= 3*/)
{
	FNumberFormattingOptions NumberFormatOptions = FNumberFormattingOptions::DefaultNoGrouping();
	NumberFormatOptions.MinimumFractionalDigits = IntegralDigits;
	NumberFormatOptions.MaximumFractionalDigits = IntegralDigits;

	FFormatNamedArguments Args;
	Args.Add(TEXT("X"), FText::AsNumber(CartesianCoordinates.X, &NumberFormatOptions));
	Args.Add(TEXT("Y"), FText::AsNumber(CartesianCoordinates.Y, &NumberFormatOptions));
	Args.Add(TEXT("Z"), FText::AsNumber(CartesianCoordinates.Z, &NumberFormatOptions));

	return FText::Format(NSLOCTEXT("GeoReferencing", "CartesianCoordinatesCompact", "({X}, {Y}, {Z}m )"), Args);
}

void UGeoReferencingBFL::ToSeparateTexts(FVector& CartesianCoordinates, FText& OutX, FText& OutY, FText& OutZ, int32 IntegralDigits /*= 3*/)
{
	FNumberFormattingOptions NumberFormatOptions = FNumberFormattingOptions::DefaultNoGrouping();
	NumberFormatOptions.MinimumFractionalDigits = IntegralDigits;
	NumberFormatOptions.MaximumFractionalDigits = IntegralDigits;

	OutX = FText::AsNumber(CartesianCoordinates.X, &NumberFormatOptions);
	OutY = FText::AsNumber(CartesianCoordinates.Y, &NumberFormatOptions);
	OutZ = FText::AsNumber(CartesianCoordinates.Z, &NumberFormatOptions);
}
