// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeoReferencingModule.h"
#include "Modules/ModuleManager.h"
#include "DataDrivenShaderPlatformInfo.h"

DEFINE_LOG_CATEGORY(LogGeoReferencing);

IMPLEMENT_MODULE(FGeoReferencingModule, GeoReferencing)


void FGeoReferencingModule::StartupModule()
{
	if (FDataDrivenShaderPlatformInfo::GetSupportSceneDataCompressedTransforms(GMaxRHIShaderPlatform))
	{
		UE_LOGF(LogGeoReferencing, Display, "The engine is currently using Compressed Rotation Transforms. You may encounter accuracy issues with large neshes. We recommend setting all bSupportsSceneDataCompressedTransforms to false in the Engine/Config/[platform]/DataDrivenPlatformInfo.ini file");
	}
}
