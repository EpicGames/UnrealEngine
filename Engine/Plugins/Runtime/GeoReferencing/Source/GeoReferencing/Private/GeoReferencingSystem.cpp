// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeoReferencingSystem.h"

#include "GameFramework/WorldSettings.h"
#include "Interfaces/IPluginManager.h"
#include "Ellipsoid.h"
#include "Kismet/GameplayStatics.h"
#include "GeoReferencingModule.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Engine/Engine.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "GeographicCoordinates.h"
#include "MathUtil.h"
#include "Components/BillboardComponent.h"
 
#if WITH_EDITOR

#include "UObject/ConstructorHelpers.h"
#include "Engine/Texture2D.h"

#endif

#include "Misc/Paths.h"
#include "UFSProjSupport.h"
#include "proj.h"

THIRD_PARTY_INCLUDES_START
THIRD_PARTY_INCLUDES_END

#define ECEF_EPSG_FSTRING FString(TEXT("EPSG:4978"))

// LWC_TODO - To be replaced once FVector::Normalize will use a smaller number than 1e-8
#define GEOREF_DOUBLE_SMALL_NUMBER			(1.e-50)



AGeoReferencingSystem::AGeoReferencingSystem(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> GeoReferencingTextureObject;
			FName ID_GeoReferencing;
			FText NAME_GeoReferencing;
			FConstructorStatics()
				: GeoReferencingTextureObject(TEXT("/GeoReferencing/UI/Icons/S_GeoReferencing"))
				, ID_GeoReferencing(TEXT("GeoReferencing"))
				, NAME_GeoReferencing(NSLOCTEXT("SpriteCategory", "GeoReferencing", "GeoReferencing"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (GetSpriteComponent())
		{
			GetSpriteComponent()->Sprite = ConstructorStatics.GeoReferencingTextureObject.Get();
			GetSpriteComponent()->SpriteInfo.Category = ConstructorStatics.ID_GeoReferencing;
			GetSpriteComponent()->SpriteInfo.DisplayName = ConstructorStatics.NAME_GeoReferencing;
		}
	}
#endif // WITH_EDITORONLY_DATA
}


AGeoReferencingSystem* AGeoReferencingSystem::GetGeoReferencingSystem(UObject* WorldContextObject)
{
	AGeoReferencingSystem* Actor = nullptr;

	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		TArray<AActor*> Actors;
		UGameplayStatics::GetAllActorsOfClass(World, AGeoReferencingSystem::StaticClass(), Actors);
		int NbActors = Actors.Num();
		if (NbActors == 0)
		{
			UE_LOGF(LogGeoReferencing, Error, "GeoReferencingSystem actor not found. Please add one to your world to configure your geo referencing system.");
		}
		else if (NbActors > 1)
		{
			UE_LOGF(LogGeoReferencing, Error, "Multiple GeoReferencingSystem actors found. Only one actor should be used to configure your geo referencing system");
		}
		else
		{
			Actor = Cast<AGeoReferencingSystem>(Actors[0]);
		}
	}

	return Actor;
}

class AGeoReferencingSystem::FGeoReferencingSystemInternals
{
public:
	FGeoReferencingSystemInternals()
		: ProjContext(nullptr)
		, ProjProjectedToGeographic(nullptr)
		, ProjProjectedToECEF(nullptr)
		, ProjGeographicToECEF(nullptr)
	{
	}

	// Private PROJ Utilities
	void InitPROJLibrary();
	void DeInitPROJLibrary();
	PJ* GetPROJProjection(FString SourceCRS, FString DestinationCRS);
	bool GetEllipsoid(FString CRSString, FEllipsoid& Ellipsoid);
	
	FMatrix GetWorldFrameToECEFFrame(const FEllipsoid& Ellipsoid, const FVector& ECEFLocation);

	PJ_CONTEXT* ProjContext;
	PJ* ProjProjectedToGeographic;
	PJ* ProjProjectedToECEF;
	PJ* ProjGeographicToECEF;
	FEllipsoid ProjectedEllipsoid;
	FEllipsoid GeographicEllipsoid;

	// Transformation caches 
	// Flat Planet
	FVector WorldOriginLocationProjected; // Offset between the UE world and the Projected CRS Origin. (Expressed in ProjectedCRS units).

	// Round Planet
	FMatrix WorldFrameToECEFFrame; // Matrix to transform a vector from EU to ECEF CRS
	FMatrix ECEFFrameToWorldFrame; // Matrix to transform a vector from ECEF to UE CRS - Inverse of the previous one kept in cache. 

	FMatrix WorldFrameToUEFrame;
	FMatrix UEFrameToWorldFrame;
};

/////// INIT / DEINIT

void AGeoReferencingSystem::PostLoad()
{
	Super::PostLoad();

	Initialize();
}

void AGeoReferencingSystem::PostActorCreated()
{
	Super::PostActorCreated();

	Initialize();
}

void AGeoReferencingSystem::Initialize()
{
	if (GetWorld() && GetWorld()->GetWorldSettings()->bEnableWorldBoundsChecks)
	{
		UE_LOGF(LogGeoReferencing, Display, "Enable World Bounds Checks is enabled in your World Settings. You might consider disabling it when working with large terrains, otherwise your pawns won't be able to go too far from the Origin");
	}


	Impl = MakePimpl<FGeoReferencingSystemInternals>();

	Impl->InitPROJLibrary();

	// Should we consider other conventions ? Or North Offset like in SunPosition?
	Impl->WorldFrameToUEFrame = FMatrix( 
		FVector(1.0, 0.0, 0.0),		// Easting (X) is UE World X
		FVector(0.0, -1.0, 0.0),	// Northing (Y) is UE World -Y because of left-handed convention
		FVector(0.0, 0.0, 1.0),		// Up (Z) is UE World Z 
		FVector(0.0, 0.0, 0.0));	// No Origin offset


	Impl->UEFrameToWorldFrame = Impl->WorldFrameToUEFrame.Inverse();

	ApplySettings();
}

void AGeoReferencingSystem::BeginDestroy()
{
	Super::BeginDestroy();

	if (Impl)
	{
		Impl->DeInitPROJLibrary();
	}
}

void AGeoReferencingSystem::EngineToProjected(const FVector& EngineCoordinates, FVector& ProjectedCoordinates)
{
	switch (PlanetShape)
	{
	case EPlanetShape::RoundPlanet:
	{
		// In RoundPlanet, we have to go through the ECEF transform as an intermediate step. 
		FVector ECEFCoordinates;
		EngineToECEF(EngineCoordinates, ECEFCoordinates);
		ECEFToProjected(ECEFCoordinates, ProjectedCoordinates);
	}
	break;

	case EPlanetShape::FlatPlanet:
	default:
	{
		// in FlatPlanet, the transform is simply a translation
		// Convert UE units to meters, invert the Y coordinate because of left-handed UE Frame
		FVector UEWorldCoordinates = EngineCoordinates * FVector(0.01, -0.01, 0.01);

		// Add the defined origin offset
		ProjectedCoordinates = UEWorldCoordinates + Impl->WorldOriginLocationProjected;
	}
	break;
	}
}

void AGeoReferencingSystem::ProjectedToEngine(const FVector& ProjectedCoordinates, FVector& EngineCoordinates)
{
	switch (PlanetShape)
	{
	case EPlanetShape::RoundPlanet:
	{
		// In RoundPlanet, we have to go through the ECEF transform as an intermediate step. 
		FVector ECEFCoordinates;
		ProjectedToECEF(ProjectedCoordinates, ECEFCoordinates);
		ECEFToEngine(ECEFCoordinates, EngineCoordinates);
	}
	break;

	case EPlanetShape::FlatPlanet:
	default:
	{
		// in FlatPlanet, the transform is simply a translation
		// Remove the Origin location, and convert to UE Units, while inverting the Z Axis
		FVector UEWorldCoordinates = (ProjectedCoordinates - Impl->WorldOriginLocationProjected);

		// Convert UE units to meters, invert the Y coordinate because of left-handed UE Frame
		EngineCoordinates = UEWorldCoordinates * FVector(100.0, -100.0, 100.0);
	}
	break;
	}
}

void AGeoReferencingSystem::EngineToECEF(const FVector& EngineCoordinates, FVector& ECEFCoordinates)
{
	switch (PlanetShape)
	{
	case EPlanetShape::RoundPlanet:
	{
		// Convert UE units to meters, invert the Y coordinate because of left-handed UE Frame
		FVector UEWorldCoordinates = EngineCoordinates * FVector(0.01, -0.01, 0.01);

		if (bOriginAtPlanetCenter)
		{
			// Easy case, UE is ECEF... return the Global coordinates
			ECEFCoordinates = UEWorldCoordinates; // TOCHECK Types
		}
		else
		{
			ECEFCoordinates = Impl->WorldFrameToECEFFrame.TransformPosition(UEWorldCoordinates);
		}
	}
	break;

	case EPlanetShape::FlatPlanet:
	default:
	{
		// In FlatPlanet, we have to go through the Projected transform as an intermediate step. 
		FVector ProjectedCoordinates;
		EngineToProjected(EngineCoordinates, ProjectedCoordinates);
		ProjectedToECEF(ProjectedCoordinates, ECEFCoordinates);
	}
	break;
	}
}

void AGeoReferencingSystem::ECEFToEngine(const FVector& ECEFCoordinates, FVector& EngineCoordinates)
{
	switch (PlanetShape)
	{
	case EPlanetShape::RoundPlanet:
	{
		FVector UEWorldCoordinates;
		if (bOriginAtPlanetCenter)
		{
			// Easy case, UE is ECEF... And we did the rebasing, so return the Global coordinates
			UEWorldCoordinates = ECEFCoordinates;
		}
		else
		{
			UEWorldCoordinates = Impl->ECEFFrameToWorldFrame.TransformPosition(ECEFCoordinates);
		}

		// Convert UE units to meters, invert the Y coordinate because of left-handed UE Frame
		EngineCoordinates = UEWorldCoordinates * FVector(100.0, -100.0, 100.0);
	}
	break;

	case EPlanetShape::FlatPlanet:
	default:
	{
		// In FlatPlanet, we have to go through the Projected transform as an intermediate step. 
		FVector ProjectedCoordinates;
		ECEFToProjected(ECEFCoordinates, ProjectedCoordinates);
		ProjectedToEngine(ProjectedCoordinates, EngineCoordinates);
	}
	break;
	}
}


void AGeoReferencingSystem::EngineToGeographic(const FVector& EngineCoordinates, FGeographicCoordinates& GeographicCoordinates)
{
	switch (PlanetShape)
	{
		case EPlanetShape::RoundPlanet:
		{
			FVector ECEFCoordinates;
			EngineToECEF(EngineCoordinates, ECEFCoordinates);
			ECEFToGeographic(ECEFCoordinates, GeographicCoordinates);
		}
	break;

		case EPlanetShape::FlatPlanet:
		default:
		{
			FVector ProjectedCoordinates;
			EngineToProjected(EngineCoordinates, ProjectedCoordinates);
			ProjectedToGeographic(ProjectedCoordinates, GeographicCoordinates);

		}
	break;
	}
}

void AGeoReferencingSystem::GeographicToEngine(const FGeographicCoordinates& GeographicCoordinates, FVector& EngineCoordinates)
{
	switch (PlanetShape)
	{
		case EPlanetShape::RoundPlanet:
		{
			FVector ECEFCoordinates;
			GeographicToECEF(GeographicCoordinates, ECEFCoordinates);
			ECEFToEngine(ECEFCoordinates, EngineCoordinates);
		}
	break;

		case EPlanetShape::FlatPlanet:
		default:
		{
			FVector ProjectedCoordinates;
			GeographicToProjected(GeographicCoordinates,ProjectedCoordinates);
			ProjectedToEngine(ProjectedCoordinates, EngineCoordinates);
		}
	break;
	}
}


void AGeoReferencingSystem::ProjectedToGeographic(const FVector& ProjectedCoordinates, FGeographicCoordinates& GeographicCoordinates)
{
	PJ_COORD input, output;
	input = proj_coord(ProjectedCoordinates.X, ProjectedCoordinates.Y, ProjectedCoordinates.Z, 0);

	output = proj_trans(Impl->ProjProjectedToGeographic, PJ_FWD, input);
	GeographicCoordinates.Latitude = output.lpz.phi;
	GeographicCoordinates.Longitude = output.lpz.lam;
	GeographicCoordinates.Altitude = output.lpz.z;
}

void AGeoReferencingSystem::GeographicToProjected(const FGeographicCoordinates& GeographicCoordinates, FVector& ProjectedCoordinates)
{
	PJ_COORD input, output;
	input = proj_coord(GeographicCoordinates.Longitude, GeographicCoordinates.Latitude, GeographicCoordinates.Altitude, 0);

	output = proj_trans(Impl->ProjProjectedToGeographic, PJ_INV, input);
	ProjectedCoordinates.X = output.xyz.x;
	ProjectedCoordinates.Y = output.xyz.y;
	ProjectedCoordinates.Z = output.xyz.z;
}

void AGeoReferencingSystem::ProjectedToECEF(const FVector& ProjectedCoordinates, FVector& ECEFCoordinates)
{
	PJ_COORD input, output;
	input = proj_coord(ProjectedCoordinates.X, ProjectedCoordinates.Y, ProjectedCoordinates.Z, 0);

	output = proj_trans(Impl->ProjProjectedToECEF, PJ_FWD, input);
	ECEFCoordinates.X = output.xyz.x;
	ECEFCoordinates.Y = output.xyz.y;
	ECEFCoordinates.Z = output.xyz.z;
}

void AGeoReferencingSystem::ECEFToProjected(const FVector& ECEFCoordinates, FVector& ProjectedCoordinates)
{
	PJ_COORD input, output;
	input = proj_coord(ECEFCoordinates.X, ECEFCoordinates.Y, ECEFCoordinates.Z, 0);

	output = proj_trans(Impl->ProjProjectedToECEF, PJ_INV, input);
	ProjectedCoordinates.X = output.xyz.x;
	ProjectedCoordinates.Y = output.xyz.y;
	ProjectedCoordinates.Z = output.xyz.z;
}

void AGeoReferencingSystem::GeographicToECEF(const FGeographicCoordinates& GeographicCoordinates, FVector& ECEFCoordinates)
{
	PJ_COORD input, output;
	input = proj_coord(GeographicCoordinates.Longitude, GeographicCoordinates.Latitude, GeographicCoordinates.Altitude, 0);

	output = proj_trans(Impl->ProjGeographicToECEF, PJ_FWD, input);
	ECEFCoordinates.X = output.xyz.x;
	ECEFCoordinates.Y = output.xyz.y;
	ECEFCoordinates.Z = output.xyz.z;
}

void AGeoReferencingSystem::ECEFToGeographic(const FVector& ECEFCoordinates, FGeographicCoordinates& GeographicCoordinates)
{
	PJ_COORD input, output;
	input = proj_coord(ECEFCoordinates.X, ECEFCoordinates.Y, ECEFCoordinates.Z, 0);

	output = proj_trans(Impl->ProjGeographicToECEF, PJ_INV, input);
	GeographicCoordinates.Latitude = output.lpz.phi;
	GeographicCoordinates.Longitude = output.lpz.lam;
	GeographicCoordinates.Altitude = output.lpz.z;
}

// ENU & Transforms

void AGeoReferencingSystem::GetENUVectorsAtEngineLocation(const FVector& EngineCoordinates, FVector& East, FVector& North, FVector& Up)
{
	FVector ECEFLocation;
	EngineToECEF(EngineCoordinates, ECEFLocation);
	GetENUVectorsAtECEFLocation(ECEFLocation, East, North, Up);
}

void AGeoReferencingSystem::GetENUVectorsAtProjectedLocation(const FVector& ProjectedCoordinates, FVector& East, FVector& North, FVector& Up)
{
	FVector ECEFLocation;
	ProjectedToECEF(ProjectedCoordinates, ECEFLocation);
	GetENUVectorsAtECEFLocation(ECEFLocation, East, North, Up);
}

void AGeoReferencingSystem::GetENUVectorsAtGeographicLocation(const FGeographicCoordinates& GeographicCoordinates, FVector& East, FVector& North, FVector& Up)
{
	FVector ECEFLocation;
	GeographicToECEF(GeographicCoordinates, ECEFLocation);
	GetENUVectorsAtECEFLocation(ECEFLocation, East, North, Up);
}

void AGeoReferencingSystem::GetENUVectorsAtECEFLocation(const FVector& ECEFCoordinates, FVector& East, FVector& North, FVector& Up)
{
	// Compute Tangent matrix at ECEF location
	FEllipsoid& Ellipsoid = Impl->GeographicEllipsoid;

	if (bOriginLocationInProjectedCRS)
	{
		Ellipsoid = Impl->ProjectedEllipsoid;
	}
	FMatrix WorldFrameToECEFFrameAtLocation;
	WorldFrameToECEFFrameAtLocation = Impl->GetWorldFrameToECEFFrame(Ellipsoid, ECEFCoordinates);

	switch (PlanetShape)
	{
	case EPlanetShape::RoundPlanet:
	{
		FMatrix UEtoECEF = WorldFrameToECEFFrameAtLocation * Impl->ECEFFrameToWorldFrame * Impl->UEFrameToWorldFrame;
		UEtoECEF.GetUnitAxes(East, North, Up);
	}
	break;

	case EPlanetShape::FlatPlanet:
	default:
		// PROJ don't provide anything to project direction vectors. Let's do it by hand...
		FVector EasternPoint = ECEFCoordinates + WorldFrameToECEFFrameAtLocation.TransformVector(FVector(1.0, 0.0, 0.0)); // 1m from origin to the East
		FVector NorthernPoint = ECEFCoordinates + WorldFrameToECEFFrameAtLocation.TransformVector(FVector(0.0, 1.0, 0.0)); // 1m from origin to the North
		
		FVector ProjectedOrigin, ProjectedEastern, ProjectedNorthern;
		ECEFToProjected(ECEFCoordinates, ProjectedOrigin);
		ECEFToProjected(EasternPoint, ProjectedEastern);
		ECEFToProjected(NorthernPoint, ProjectedNorthern);

		FVector EastDirection(ProjectedEastern.X - ProjectedOrigin.X, ProjectedEastern.Y - ProjectedOrigin.Y, ProjectedEastern.Z - ProjectedOrigin.Z);
		FVector NorthDirection(ProjectedNorthern.X - ProjectedOrigin.X, ProjectedNorthern.Y - ProjectedOrigin.Y, ProjectedNorthern.Z - ProjectedOrigin.Z);

		EastDirection.Normalize(GEOREF_DOUBLE_SMALL_NUMBER);
		NorthDirection.Normalize(GEOREF_DOUBLE_SMALL_NUMBER);

		East = FVector(EastDirection.X, -EastDirection.Y, EastDirection.Z);
		North = FVector(NorthDirection.X, -NorthDirection.Y, NorthDirection.Z);
		Up = FVector::CrossProduct(North, East);
		break;
	}
}

void AGeoReferencingSystem::GetECEFENUVectorsAtECEFLocation(const FVector& ECEFCoordinates, FVector& ECEFEast, FVector& ECEFNorth, FVector& ECEFUp)
{
	// Compute Tangent matrix at ECEF location
	FEllipsoid& Ellipsoid = Impl->GeographicEllipsoid;
	
	if (bOriginLocationInProjectedCRS)
	{
		Ellipsoid = Impl->ProjectedEllipsoid;
	}
	
	FMatrix WorldFrameToECEFFrameAtLocation;
	WorldFrameToECEFFrameAtLocation = Impl->GetWorldFrameToECEFFrame(Ellipsoid, ECEFCoordinates);
	WorldFrameToECEFFrameAtLocation.GetUnitAxes(ECEFEast, ECEFNorth, ECEFUp);
}

FTransform AGeoReferencingSystem::GetTangentTransformAtEngineLocation(const FVector& EngineCoordinates)
{
	FVector ECEFLocation;
	EngineToECEF(EngineCoordinates, ECEFLocation);
	return GetTangentTransformAtECEFLocation(ECEFLocation);
}

FTransform AGeoReferencingSystem::GetTangentTransformAtProjectedLocation(const FVector& ProjectedCoordinates)
{
	FVector ECEFLocation;
	ProjectedToECEF(ProjectedCoordinates, ECEFLocation);
	return GetTangentTransformAtECEFLocation(ECEFLocation);
}

FTransform AGeoReferencingSystem::GetTangentTransformAtGeographicLocation(const FGeographicCoordinates& GeographicCoordinates)
{
	FVector ECEFLocation;
	GeographicToECEF(GeographicCoordinates, ECEFLocation);
	return GetTangentTransformAtECEFLocation(ECEFLocation);
}

FTransform AGeoReferencingSystem::GetTangentTransformAtECEFLocation(const FVector& ECEFCoordinates)
{
	if (PlanetShape == EPlanetShape::RoundPlanet)
	{
		// Compute Tangent matrix at ECEF location
		FEllipsoid& Ellipsoid = Impl->GeographicEllipsoid;
		if (bOriginLocationInProjectedCRS)
		{
			Ellipsoid = Impl->ProjectedEllipsoid;
		}
		FMatrix WorldFrameToECEFFrameAtLocation = Impl->GetWorldFrameToECEFFrame(Ellipsoid, ECEFCoordinates);
		

		FMatrix UEtoECEF = Impl->UEFrameToWorldFrame * WorldFrameToECEFFrameAtLocation * Impl->ECEFFrameToWorldFrame * Impl->WorldFrameToUEFrame; 
		FVector UEOrigin;
		ECEFToEngine(ECEFCoordinates, UEOrigin);
		UEtoECEF.SetOrigin(UEOrigin);

		return FTransform(UEtoECEF);
	}
	else
	{
		FVector East, North, Up, UEOrigin;
		GetENUVectorsAtECEFLocation(ECEFCoordinates, East, North, Up);
		ECEFToEngine(ECEFCoordinates, UEOrigin);
		FMatrix TransformMatrix(East, North, Up, UEOrigin);
		return FTransform(TransformMatrix);
	}
}

FTransform AGeoReferencingSystem::GetPlanetCenterTransform()
{
	// Compute Origin location in ECEF. 
	if (PlanetShape == EPlanetShape::RoundPlanet)
	{
		if (bOriginAtPlanetCenter)
		{
			return FTransform::Identity;
		}
		else
		{
			FMatrix TransformMatrix = Impl->UEFrameToWorldFrame * Impl->ECEFFrameToWorldFrame * Impl->WorldFrameToUEFrame;
			// Don't go to transform yet, we must stay in double to apply the rebasing offset. 

			// Get Origin, and convert UE units to meters, invert the Y coordinate because of left-handed UE Frame
			FVector UEOrigin = TransformMatrix.GetOrigin() * FVector(100.0, 100.0, 100.0);
			TransformMatrix.SetOrigin(UEOrigin);

			return FTransform(TransformMatrix);
		}
	}
	else
	{
		// Makes not sense in Flat planet mode... 
		return FTransform::Identity;
	}
}

// Public PROJ Utilities

bool AGeoReferencingSystem::IsCRSStringValid(FString CRSString, FString& Error)
{
	if (Impl->ProjContext == nullptr)
	{
		Error = FString("Proj Context has not been initialized");
		return false;
	}

	// Try to create a CRS from this string
	FTCHARToUTF8 Convert(*CRSString);
	const ANSICHAR* UtfString = Convert.Get();
	PJ* CRS = proj_create(Impl->ProjContext, UtfString);

	if (CRS == nullptr)
	{
		int ErrorNumber = proj_context_errno(Impl->ProjContext);
		Error = FString(proj_errno_string(ErrorNumber));
		return false;
	}

	proj_destroy(CRS);
	return true;
}

// Ellipsoid

double AGeoReferencingSystem::GetGeographicEllipsoidMaxRadius()
{
	return Impl->GeographicEllipsoid.GetMaximumRadius();
}

double AGeoReferencingSystem::GetGeographicEllipsoidMinRadius()
{
	return Impl->GeographicEllipsoid.GetMinimumRadius();
}

double AGeoReferencingSystem::GetProjectedEllipsoidMaxRadius()
{
	return Impl->ProjectedEllipsoid.GetMaximumRadius();
}

double AGeoReferencingSystem::GetProjectedEllipsoidMinRadius()
{
	return Impl->ProjectedEllipsoid.GetMinimumRadius();
}

bool AGeoReferencingSystem::FGeoReferencingSystemInternals::GetEllipsoid(FString CRSString, FEllipsoid& Ellipsoid)
{
	FTCHARToUTF8 ConvertCRSString(*CRSString);
	const ANSICHAR* CRS = ConvertCRSString.Get();
	bool bSuccess = true;

	PJ* CRSPJ = proj_create(ProjContext, CRS);
	if (CRSPJ != nullptr)
	{
		PJ* EllipsoidPJ = proj_get_ellipsoid(ProjContext, CRSPJ);
		if (EllipsoidPJ != nullptr)
		{
			double SemiMajorMetre; // semi-major axis in meter
			double SemiMinorMetre; // semi-minor axis in meter
			double InvFlattening;  // inverse flattening.
			int IsSemiMinorComputed; // if the semi-minor value was computed. If FALSE, its value comes from the definition

			if (proj_ellipsoid_get_parameters(ProjContext, EllipsoidPJ, &SemiMajorMetre, &SemiMinorMetre, &IsSemiMinorComputed, &InvFlattening))
			{
				Ellipsoid = FEllipsoid(SemiMajorMetre, SemiMajorMetre, SemiMinorMetre);
			}
			else
			{
				int ErrorNumber = proj_context_errno(ProjContext);
				FString ProjError = FString(proj_errno_string(ErrorNumber));
				UE_LOGF(LogGeoReferencing, Error, "AGeoReferencingSystem::GetEllipsoid failed in proj_ellipsoid_get_parameters : %ls ", *ProjError);
				bSuccess = false;
			}

			proj_destroy(EllipsoidPJ);
		}
		else
		{
			int ErrorNumber = proj_context_errno(ProjContext);
			FString ProjError = FString(proj_errno_string(ErrorNumber));
			UE_LOGF(LogGeoReferencing, Error, "AGeoReferencingSystem::GetEllipsoid failed in proj_get_ellipsoid : %ls ", *ProjError);
			bSuccess = false;
		}

		proj_destroy(CRSPJ);
	}
	else
	{
		int ErrorNumber = proj_context_errno(ProjContext);
		FString ProjError = FString(proj_errno_string(ErrorNumber));
		UE_LOGF(LogGeoReferencing, Error, "AGeoReferencingSystem::GetEllipsoid failed in proj_create : %ls ", *ProjError);
		UE_LOGF(LogGeoReferencing, Display, "CRSString was : %ls ", *CRSString);
		bSuccess = false;
	}

	return bSuccess;
}

void AGeoReferencingSystem::ApplySettings()
{
	// Apply Projection settings

	// Projected -> Geographic
	if (Impl->ProjProjectedToGeographic != nullptr)
	{
		proj_destroy(Impl->ProjProjectedToGeographic);
	}
	Impl->ProjProjectedToGeographic = Impl->GetPROJProjection(ProjectedCRS, GeographicCRS);

	// Projected -> Geocentric
	if (Impl->ProjProjectedToECEF != nullptr)
	{
		proj_destroy(Impl->ProjProjectedToECEF);
	}
	Impl->ProjProjectedToECEF = Impl->GetPROJProjection(ProjectedCRS, ECEF_EPSG_FSTRING);

	// Geographic -> Geocentric
	if (Impl->ProjGeographicToECEF != nullptr)
	{
		proj_destroy(Impl->ProjGeographicToECEF);
	}
	Impl->ProjGeographicToECEF = Impl->GetPROJProjection(GeographicCRS, ECEF_EPSG_FSTRING);

	bool bProjectedEllipsoidSuccess = Impl->GetEllipsoid(ProjectedCRS, Impl->ProjectedEllipsoid);
	bool bGeographicEllipsoidSuccess = Impl->GetEllipsoid(GeographicCRS, Impl->GeographicEllipsoid);

	bool bSuccess = Impl->ProjProjectedToGeographic != nullptr && Impl->ProjProjectedToECEF != nullptr && Impl->ProjGeographicToECEF != nullptr && bProjectedEllipsoidSuccess && bGeographicEllipsoidSuccess;

#if WITH_EDITOR
	if (!bSuccess)
	{
		// Show an error notification
		const FText NotificationErrorText = NSLOCTEXT("GeoReferencing", "GeoReferencingCRSError", "Error in one CRS definition string - Check log");
		FNotificationInfo Info(NotificationErrorText);
		Info.ExpireDuration = 2.0f;
		Info.bUseSuccessFailIcons = true;
		FSlateNotificationManager::Get().AddNotification(Info);
		return;
	}
#endif

	// Apply Origin settings

	switch (PlanetShape)
	{
	case EPlanetShape::RoundPlanet:
		// Matrice UE to ECEF :
		if (bOriginAtPlanetCenter)
		{
			// Theoritically, we should never use this Identity matrices since the transformations already handle that using a shorter code path, but let's keep consistency  
			Impl->ECEFFrameToWorldFrame.SetIdentity();
			Impl->WorldFrameToECEFFrame.SetIdentity();
		}
		else
		{
			// We need to compute the ENU Vectors at the origin point. For that, the origin has to be expressed first in ECEF. 
			// Express origin in ECEF, and get the ENU vectors
			FVector ECEFOrigin;
			Impl->WorldFrameToECEFFrame.SetIdentity();
			if (bOriginLocationInProjectedCRS)
			{
				FVector ProjectedOrigin(
					OriginProjectedCoordinatesEasting,
					OriginProjectedCoordinatesNorthing,
					OriginProjectedCoordinatesUp);
				ProjectedToECEF(ProjectedOrigin, ECEFOrigin);

				Impl->WorldFrameToECEFFrame = Impl->GetWorldFrameToECEFFrame(Impl->ProjectedEllipsoid, ECEFOrigin);
			}
			else
			{
				GeographicToECEF(FGeographicCoordinates(OriginLongitude, OriginLatitude, OriginAltitude), ECEFOrigin);
				Impl->WorldFrameToECEFFrame = Impl->GetWorldFrameToECEFFrame(Impl->GeographicEllipsoid, ECEFOrigin);
			}
			Impl->ECEFFrameToWorldFrame = Impl->WorldFrameToECEFFrame.Inverse();
		}
		break;

	case EPlanetShape::FlatPlanet:
	default:
		if (bOriginLocationInProjectedCRS)
		{
			// World origin is expressed using Projected coordinates. Take them as is (in double)
			Impl->WorldOriginLocationProjected = FVector(
				OriginProjectedCoordinatesEasting,
				OriginProjectedCoordinatesNorthing,
				OriginProjectedCoordinatesUp);
		}
		else
		{
			// World origin is expressed using Geographic coordinates. Convert them to the projected CRS to have the offset
			FVector OriginProjected;
			GeographicToProjected(FGeographicCoordinates(OriginLongitude, OriginLatitude, OriginAltitude), OriginProjected);
			Impl->WorldOriginLocationProjected = FVector(OriginProjected.X, OriginProjected.Y, OriginProjected.Z);
		}
		break;
	}
	return;
}

static void ProjLog(void* app_data, int level, const char* message)
{
	FUTF8ToTCHAR Message (message);
	switch (level)
	{
	case PJ_LOG_ERROR:
		UE_LOGF(LogGeoReferencing, Error, "%ls", Message.Get());
		break;
	case PJ_LOG_DEBUG:
		UE_LOGF(LogGeoReferencing, Verbose, "%ls", Message.Get());
		break;
	case PJ_LOG_TRACE:
		UE_LOGF(LogGeoReferencing, VeryVerbose, "%ls", Message.Get());
		break;
	}
}

void AGeoReferencingSystem::FGeoReferencingSystemInternals::InitPROJLibrary()
{
	// Initialize proj context
	ProjContext = proj_context_create();
	if (ProjContext == nullptr)
	{
		UE_LOGF(LogGeoReferencing, Error, "proj_context_create() failed");
		return;
	}

	// Connect PROJ logging
	proj_log_func(ProjContext, nullptr, ProjLog);
	proj_log_level(ProjContext, PJ_LOG_TRACE);

	// Calculate and register the search path to the PROJ data
	FString PluginBaseDir = IPluginManager::Get().FindPlugin("GeoReferencing")->GetBaseDir();
	FString ProjDataPath = FPaths::Combine(*PluginBaseDir, TEXT("Resources/PROJ"));

	UE_LOGF(LogGeoReferencing, Display, "Setting search path in %ls ", *ProjDataPath);


	FTCHARToUTF8 Utf8ProjDataPath(*ProjDataPath);
	const char* ProjSearchPaths[] =
	{
		Utf8ProjDataPath.Get(),
	};
	proj_context_set_search_paths(ProjContext, sizeof(ProjSearchPaths)/sizeof(ProjSearchPaths[0]), ProjSearchPaths);

	// With PROJ 9.1.1 this function does nothing
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	proj_context_set_autoclose_database(ProjContext, true);
	FString ProjDBFilePath = FPaths::Combine(*ProjDataPath, TEXT("proj.db"));
	PlatformFile.SetReadOnly(*ProjDBFilePath, true); // Set it read only so that with PROJ 9.1.1 SQLite doesn't open the file with a write lock
	
	// Special case for Dedicated Server or client
	// When running in a sandboxed environment, the PROJ data files are not staged in that environment by the Cooking process. 
	// And our SQLite accessor will always want to find these files in the sandboxed location, never in the original one. 
	// And we don't want to have to build the PAK files when iterating on the code
	// Let's manually populate the sandbox environment. 
	if (PlatformFile.IsSandboxEnabled())
	{
		// Ask the system for the expected location of the proj.db file, using ConvertToAbsolutePathForExternalAppForWrite (not ***Read, that falls back on the original location)
		FString ProjDBFileSandboxPath = PlatformFile.ConvertToAbsolutePathForExternalAppForWrite(*ProjDBFilePath);
		UE_LOGF(LogGeoReferencing, Warning, "Running in Sandboxed mode with Proj.db expected in %ls ", *ProjDBFileSandboxPath);

		// Check if the file is missing there
		if (!IFileManager::Get().FileExists(*ProjDBFileSandboxPath))
		{
			UE_LOGF(LogGeoReferencing, Warning, "Missing PROJ data files in then Sandbox environment --> Copy them!");

			// Create the target dir, and deploy the content. 
			FString TargetFolder = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*ProjDataPath); // Use **Write for the actual Sandboxed path. 
			PlatformFile.GetLowerLevel()->CreateDirectoryTree(*TargetFolder); // Use lower level Platformfile (otherwise the sandbox path is added twice)
			PlatformFile.GetLowerLevel()->CopyDirectoryTree(*TargetFolder, *ProjDataPath, false);

			PlatformFile.SetReadOnly(*ProjDBFileSandboxPath, true); // Set it read only so that with PROJ 9.1.1 SQLite doesn't open the file with a write lock
		}
	}
	// Non-editor builds use UFS extensions to read PROJ data from UFS/Pak
	if (!GIsEditor)
	{
		// Connect the UFS support for SQLite to PROJ
		proj_context_set_sqlite3_vfs_name(ProjContext, "unreal-fs");

		// Setup UFS for PROJ
		if (!proj_context_set_fileapi(ProjContext, &FUFSProj::FunctionTable, nullptr))
		{
			UE_LOGF(LogGeoReferencing, Error, "proj_context_set_fileapi() failed");
		}
	}
}

void AGeoReferencingSystem::FGeoReferencingSystemInternals::DeInitPROJLibrary()
{
	// Destroy projections
	if (ProjProjectedToGeographic != nullptr)
	{
		proj_destroy(ProjProjectedToGeographic);
		ProjProjectedToGeographic = nullptr;
	}
	if (ProjProjectedToECEF != nullptr)
	{
		proj_destroy(ProjProjectedToECEF);
		ProjProjectedToECEF = nullptr;
	}
	if (ProjGeographicToECEF != nullptr)
	{
		proj_destroy(ProjGeographicToECEF);
		ProjGeographicToECEF = nullptr;
	}

	// Destroy proj context
	if (ProjContext != nullptr)
	{
		proj_context_destroy(ProjContext); /* may be omitted in the single threaded case */
		ProjContext = nullptr;
	}
}

PJ* AGeoReferencingSystem::FGeoReferencingSystemInternals::GetPROJProjection(FString SourceCRS, FString DestinationCRS)
{
	FTCHARToUTF8 ConvertSource(*SourceCRS);
	FTCHARToUTF8 ConvertDestination(*DestinationCRS);
	const ANSICHAR* Source = ConvertSource.Get();
	const ANSICHAR* Destination = ConvertDestination.Get();

	PJ* TempPJ = proj_create_crs_to_crs(ProjContext, Source, Destination, nullptr);
	if (TempPJ == nullptr)
	{
		int ErrorNumber = proj_context_errno(ProjContext);
		FString ProjError = FString(proj_errno_string(ErrorNumber));
		UE_LOGF(LogGeoReferencing, Error, "AGeoReferencingSystem::BuildProjection failed in proj_create_crs_to_crs : %ls ", *ProjError);

		UE_LOGF(LogGeoReferencing, Display, "SourceCRS was : %ls ", *SourceCRS);
		UE_LOGF(LogGeoReferencing, Display, "DestinationCRS was : %ls ", *DestinationCRS);

		return nullptr;
	}

	/* This will ensure that the order of coordinates for the input CRS */
	/* will be longitude, latitude, whereas EPSG:4326 mandates latitude, longitude */
	PJ* P_for_GIS = proj_normalize_for_visualization(ProjContext, TempPJ);
	if (P_for_GIS == nullptr)
	{
		int ErrorNumber = proj_context_errno(ProjContext);
		FString ProjError = FString(proj_errno_string(ErrorNumber));
		UE_LOGF(LogGeoReferencing, Error, "AGeoReferencingSystem::BuildProjection failed in proj_normalize_for_visualization : %ls ", *ProjError);
	}

	proj_destroy(TempPJ);
	return P_for_GIS;
}

FMatrix AGeoReferencingSystem::FGeoReferencingSystemInternals::GetWorldFrameToECEFFrame(const FEllipsoid& Ellipsoid, const FVector& ECEFLocation)
{
	// See ECEF standard : https://commons.wikimedia.org/wiki/File:ECEF_ENU_Longitude_Latitude_right-hand-rule.svg
	if (FMathd::Abs(ECEFLocation.X) < FMathd::Epsilon &&
		FMathd::Abs(ECEFLocation.Y) < FMathd::Epsilon)
	{
		// Special Case - On earth axis... 
		double Sign = 1.0;
		if (FMathd::Abs(ECEFLocation.Z) < FMathd::Epsilon)
		{
			// At origin - Should not happen, but consider it's the same as north pole
			// Leave Sign = 1
		}
		else
		{
			// At South or North pole - Axis are set to be continuous with other points
			Sign = FMathd::SignNonZero(ECEFLocation.Z);
		}

		return FMatrix(
			FVector::YAxisVector, 			// East = Y
			-FVector::XAxisVector * Sign,	// North = Sign * X
			FVector::ZAxisVector * Sign,	// Up = Sign*Z
			ECEFLocation);
	}
	else
	{
		FVector Up = Ellipsoid.GeodeticSurfaceNormal(ECEFLocation);
		FVector East(-ECEFLocation.Y, ECEFLocation.X, 0.0); 
		East.Normalize(GEOREF_DOUBLE_SMALL_NUMBER);
		FVector North = Up.Cross(East);
		return FMatrix(	East, North, Up, ECEFLocation);
	}
}

#if WITH_EDITOR
void AGeoReferencingSystem::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	//Get the name of the property that was changed  
	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(AGeoReferencingSystem, ProjectedCRS) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(AGeoReferencingSystem, GeographicCRS) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(AGeoReferencingSystem, bOriginAtPlanetCenter) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(AGeoReferencingSystem, bOriginLocationInProjectedCRS) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(AGeoReferencingSystem, OriginLatitude) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(AGeoReferencingSystem, OriginLongitude) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(AGeoReferencingSystem, OriginAltitude) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(AGeoReferencingSystem, OriginProjectedCoordinatesEasting) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(AGeoReferencingSystem, OriginProjectedCoordinatesNorthing) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(AGeoReferencingSystem, OriginProjectedCoordinatesUp) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(AGeoReferencingSystem, PlanetShape))
	{
		ApplySettings();
	}

	// Call the base class version  
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif
