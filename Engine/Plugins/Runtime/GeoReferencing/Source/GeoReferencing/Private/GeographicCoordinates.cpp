// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeographicCoordinates.h"
#include "MathUtil.h"

FGeographicCoordinates::FGeographicCoordinates()
	: Longitude(0)
	, Latitude(0)
	, Altitude(0)
{}

FGeographicCoordinates::FGeographicCoordinates(double InLongitude, double InLatitude, double InAltitude)
	: Longitude(FMathd::Clamp(InLongitude, -180.0, 180.0))
	, Latitude(FMathd::Clamp(InLatitude, -90.0, 90.0))
	, Altitude(InAltitude)
{
}

FGeographicCoordinates::FGeographicCoordinates(const FVector& LatLongAltVector)
	: Longitude(FMathd::Clamp(LatLongAltVector.Y, -180.0, 180.0))
	, Latitude(FMathd::Clamp(LatLongAltVector.X, -90.0, 90.0))
	, Altitude(LatLongAltVector.Z)
{}

FText FGeographicCoordinates::ToFullText(int32 IntegralDigitsLatLon /*= 8*/, int32 IntegralDigitsAlti /*= 2*/, bool bAsDMS /*= false*/)
{
	FNumberFormattingOptions NumberFormatOptions = FNumberFormattingOptions::DefaultNoGrouping();
	FFormatNamedArguments Args;

	// LatLon
	NumberFormatOptions.MinimumFractionalDigits = IntegralDigitsLatLon;
	NumberFormatOptions.MaximumFractionalDigits = IntegralDigitsLatLon;
	Args.Add(TEXT("Latitude"), AsAngle(Latitude, &NumberFormatOptions, bAsDMS));
	Args.Add(TEXT("Longitude"), AsAngle(Longitude, &NumberFormatOptions, bAsDMS));
	
	// Alti
	NumberFormatOptions.MinimumFractionalDigits = IntegralDigitsAlti;
	NumberFormatOptions.MaximumFractionalDigits = IntegralDigitsAlti;
	Args.Add(TEXT("Altitude"), FText::AsNumber(Altitude, &NumberFormatOptions));

	return FText::Format(NSLOCTEXT("GeoReferencing", "GeographicCoordinatesFullText", "Latitude={Latitude} Longitude={Longitude} Altitude={Altitude}m"), Args);
}

FText FGeographicCoordinates::ToCompactText(int32 IntegralDigitsLatLon /*= 8*/, int32 IntegralDigitsAlti /*= 2*/, bool bAsDMS /*= false*/)
{
	FNumberFormattingOptions NumberFormatOptions = FNumberFormattingOptions::DefaultNoGrouping();
	FFormatNamedArguments Args;

	// LatLon
	NumberFormatOptions.MinimumFractionalDigits = IntegralDigitsLatLon;
	NumberFormatOptions.MaximumFractionalDigits = IntegralDigitsLatLon;
	Args.Add(TEXT("Latitude"), AsAngle(Latitude, &NumberFormatOptions, bAsDMS));
	Args.Add(TEXT("Longitude"), AsAngle(Longitude, &NumberFormatOptions, bAsDMS));
	
	// Alti
	NumberFormatOptions.MinimumFractionalDigits = IntegralDigitsAlti;
	NumberFormatOptions.MaximumFractionalDigits = IntegralDigitsAlti;
	Args.Add(TEXT("Altitude"), FText::AsNumber(Altitude, &NumberFormatOptions));

	return FText::Format(NSLOCTEXT("GeoReferencing", "GeographicCoordinatesCompactText", "({Latitude}, {Longitude})  {Altitude}m"), Args);
}

void FGeographicCoordinates::ToSeparateTexts(FText& OutLatitude, FText& OutLongitude, FText& OutAltitude, int32 IntegralDigitsLatLon /*= 8*/, int32 IntegralDigitsAlti /*= 2*/, bool bAsDMS /*= false*/)
{
	FNumberFormattingOptions NumberFormatOptions = FNumberFormattingOptions::DefaultNoGrouping();

	// LatLon
	NumberFormatOptions.MinimumFractionalDigits = IntegralDigitsLatLon;
	NumberFormatOptions.MaximumFractionalDigits = IntegralDigitsLatLon;
	OutLatitude = AsAngle(Latitude, &NumberFormatOptions, bAsDMS);
	OutLongitude = AsAngle(Longitude, &NumberFormatOptions, bAsDMS);
	
	// Alti
	NumberFormatOptions.MinimumFractionalDigits = IntegralDigitsAlti;
	NumberFormatOptions.MaximumFractionalDigits = IntegralDigitsAlti;
	OutAltitude = FText::AsNumber(Altitude, &NumberFormatOptions);
}

FText FGeographicCoordinates::AsAngle(double Val, const FNumberFormattingOptions* const Options /*= NULL*/, bool bAsDMS /*= false*/)
{
	if (bAsDMS)
	{
		int32 degrees = FMath::TruncToInt(Val);
		int32 minutes = FMath::TruncToInt(60.0 * FMath::Abs<double>(Val - degrees));
		double seconds = 3600.0 * FMath::Abs<double>(Val - degrees) - 60.0 * minutes;
		return FText::Format(NSLOCTEXT("GeoReferencing", "AngleDMSFmt", "{0}° {1}' {2}\""), FText::AsNumber(degrees), FText::AsNumber(minutes), FText::AsNumber(seconds, Options));
	}
	else
	{
		return FText::Format(NSLOCTEXT("GeoReferencing", "AngleDegFmt", "{0}°"), FText::AsNumber(Val, Options));
	}
}
