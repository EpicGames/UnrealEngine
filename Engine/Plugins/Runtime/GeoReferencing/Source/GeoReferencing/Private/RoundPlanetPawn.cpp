// Copyright Epic Games, Inc. All Rights Reserved.


#include "RoundPlanetPawn.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "GeoReferencingSystem.h"
#include "GameFramework/FloatingPawnMovement.h"
#include "GameFramework/PlayerInput.h"
#include "GameFramework/Controller.h"
#include "Components/InputComponent.h"
#include "Curves/CurveFloat.h"
#include "GeoReferencingModule.h"
#include "GeographicCoordinates.h"
#include "Math/RotationMatrix.h"


ARoundPlanetPawn::ARoundPlanetPawn() : ADefaultPawn()
{
	PrimaryActorTick.bCanEverTick = true;
}

void ARoundPlanetPawn::FlyToLocationECEF(const FVector& ECEFDestination, double YawAtDestination, double PitchAtDestination, bool CanInterruptByMoving)
{
	if (!GeoReferencingSystem)
	{
		return;
	}
	FGeographicCoordinates TempGeographic;
	FVector TempLocation;

	// We will work on position in ECEF space, because origin is at planet center, that makes some quaternion-based maths easier. 
	
	// Compute Origin location in ECEF space
	FVector ECEFOrigin;
	GeoReferencingSystem->EngineToECEF(GetActorLocation(), ECEFOrigin);

	// Compute the source and destination rotations in ENU
	// As during the flight, we can go around the globe, this is better to interpolate in ENU coordinates
	FlyToLocationOriginRotation = ADefaultPawn::GetViewRotation();
	FlyToLocationDestinationRotation = FRotator(PitchAtDestination, YawAtDestination, 0);

	// Basically, the motion from Origin to Destination is a the shorted rotation around an axis
	// This is what a quaternion allow to do easily ! 
	// Compute Rotation Axis/Angle transform and initialize key points 
	FQuat TrajectoryQuat = FQuat::FindBetweenVectors(ECEFOrigin, ECEFDestination); // We can do that because ECEF Origin is Planet center 
	double TrajectoryTotalAngleRadians;
	FVector TrajectoryRotationAxis;
	TrajectoryQuat.ToAxisAndAngle(TrajectoryRotationAxis, TrajectoryTotalAngleRadians);

	// Avoid heavy computations during the flight. Pre-generate a fixed list of keypoint, to be interpolated during the motion
	// Avoid a too small amount of keypoints. 
	int TrajectorySteps = FMath::Max(int(TrajectoryTotalAngleRadians / FMath::DegreesToRadians(GranularityDegrees)) - 1, 0);
	TrajectorySteps = FMath::Clamp(TrajectorySteps, MinimumStepCount, MaximumStepCount);
	FlyToLocationKeypoints.Empty(TrajectorySteps + 2); 

	// We will not create a curve projected along the ellipsoid as we want to get altitude over terrain while flying. 
	// The radius of the current point will evolve as follow
	//  - Project the point on the ellipsoid - Will give a default radius depending on ellipsoid location. 
	//  - Interpolate the altitudes : get origin/destination altitude, and make a linear interpolation between them. This will allow for flying from/to any point smoothly. 
	//  - Add as flightProfile offset /-\ defined by a curve. 

	// Compute global radius at source and destination points
	FVector OriginUpECEF, DestinationUpECEF;
	float OriginRadius, DestinationRadius;
	ECEFOrigin.ToDirectionAndLength(OriginUpECEF, OriginRadius);
	ECEFDestination.ToDirectionAndLength(DestinationUpECEF, DestinationRadius);

	// Compute Altitude at origin and destination points by using geographic coordinates
	float OriginAltitude, DestinationAltitude = 0;
	GeoReferencingSystem->ECEFToGeographic(ECEFOrigin, TempGeographic);
	OriginAltitude = TempGeographic.Altitude;
	GeoReferencingSystem->ECEFToGeographic(ECEFDestination, TempGeographic);
	DestinationAltitude = TempGeographic.Altitude;
	
	// Get distance between source and destination points to compute a wanted altitude from curve
	// This is not a distance on the ellipsoid just a direct distance between two points to have a ROM of the travel length
	float TravelDirectDistance = FVector::Dist(ECEFOrigin, ECEFDestination);

	// Add first keypoint
	GeoReferencingSystem->ECEFToEngine(ECEFOrigin, TempLocation);
	FlyToLocationKeypoints.Add(TempLocation);
	
	// Debug
	//DrawDebugPoint(GetWorld(), TempLocation, 8, FColor::Red, true, 30);

	for (int CurrentStep = 1; CurrentStep <= TrajectorySteps; CurrentStep++)
	{
		float TravelPercentage = (float)CurrentStep / (TrajectorySteps + 1);

		// Lerp over Origin and Destination altitude to account for Altitude changes 
		float ExtrapolatedAltitude = FMath::Lerp<float>(OriginAltitude, DestinationAltitude, TravelPercentage);
		float TravelAngle = static_cast<float>(CurrentStep) * FMath::RadiansToDegrees(TrajectoryTotalAngleRadians) / ( TrajectorySteps + 1);

		// Compute the current point by rotating by an angle step
		FVector RotatedOrigin = ECEFOrigin.RotateAngleAxis(TravelAngle, TrajectoryRotationAxis);
		
		// Project on the Ellipsoid / Neutralize Altitude
		GeoReferencingSystem->ECEFToGeographic(RotatedOrigin, TempGeographic);
		TempGeographic.Altitude = 0;
		FVector RotatedProjectedOrigin;
		GeoReferencingSystem->GeographicToECEF(TempGeographic, RotatedProjectedOrigin);

		// Get Local Up vector
		FVector LocalUp = RotatedOrigin - RotatedProjectedOrigin;
		LocalUp.Normalize();

		// Add an altitude if we have a profile curve for it
		float AltitudeOffset = 0;
		if (AltitudeProfileCurve != NULL)
		{
			float ReferenceAltitude = MaximumAltitudeValue;
			if (MaximumAltitudeCurve != NULL)
			{
				ReferenceAltitude = MaximumAltitudeCurve->GetFloatValue(TravelDirectDistance);
			}
			AltitudeOffset = ReferenceAltitude * AltitudeProfileCurve->GetFloatValue(TravelPercentage);
		}

		// Compute the final keypoint
		FVector FinalPoint = RotatedProjectedOrigin + LocalUp * (ExtrapolatedAltitude + AltitudeOffset);
		GeoReferencingSystem->ECEFToEngine(FinalPoint, TempLocation); 
		FlyToLocationKeypoints.Add(TempLocation);

		// Debug
		//DrawDebugPoint(GetWorld(), TempLocation, 8, FColor::Purple, true, 30);
	}
	GeoReferencingSystem->ECEFToEngine(ECEFDestination, TempLocation);
	FlyToLocationKeypoints.Add(TempLocation);
	
	// Debug
	//DrawDebugPoint(GetWorld(), TempLocation, 8, FColor::Purple, true, 30);

	// Tell the tick we will be flying from now
	bIsFlyingToLocation = true;
	bCanInterruptFlight = CanInterruptByMoving;
	CurrentFlyTime = 0;
}

void ARoundPlanetPawn::FlyToLocationGeographic(const FGeographicCoordinates& GeographicDestination, double YawAtDestination, double PitchAtDestination, bool CanInterruptByMoving)
{
	if (GeoReferencingSystem)
	{
		FVector ECEFDestination;
		GeoReferencingSystem->GeographicToECEF(GeographicDestination, ECEFDestination);
		FlyToLocationECEF(ECEFDestination, YawAtDestination, PitchAtDestination, CanInterruptByMoving);
	}

}

void ARoundPlanetPawn::FlyToLocationLatitudeLongitudeAltitude(const double& InLatitude, const double& InLongitude, const double& InAltitude, double YawAtDestination, double PitchAtDestination, bool CanInterruptByMoving)
{
	FlyToLocationGeographic(FGeographicCoordinates(InLongitude, InLatitude, InAltitude), YawAtDestination, PitchAtDestination, CanInterruptByMoving);
}

void ARoundPlanetPawn::InterruptFlyToLocation()
{
	this->bIsFlyingToLocation = false;

	// fix camera roll to 0.0
	FRotator currentRotator = GetController()->GetControlRotation();
	currentRotator.Roll = 0.0;
	GetController()->SetControlRotation(currentRotator);
}

// Public Overridables

void ARoundPlanetPawn::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);
	if (!GeoReferencingSystem)
	{
		GeoReferencingSystem = AGeoReferencingSystem::GetGeoReferencingSystem(GetWorld());
	}
}

void ARoundPlanetPawn::BeginPlay()
{
	Super::BeginPlay();

	if (!GeoReferencingSystem)
	{
		GeoReferencingSystem = AGeoReferencingSystem::GetGeoReferencingSystem(GetWorld());

		if (!GeoReferencingSystem)
		{
			UE_LOGF(LogGeoReferencing, Error, "Impossible to use a RoundPlanetPawn without a GeoReferencingSystem.");
		}
	}

	// Update cache at first run
	UpdateMotionObjectsCache(true);
}

void ARoundPlanetPawn::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);
	
	UpdateMotionObjectsCache();
	HandleMotionSpeed();
	ProcessFlyToLocation(DeltaSeconds);
}

void ARoundPlanetPawn::MoveForward(float Val)
{
	if (Val != 0.f)
	{
		if (GetController())
		{
			FRotator const ControlSpaceRot = this->GetViewRotation();

			FVector MovementVector = FRotationMatrix(ControlSpaceRot).GetScaledAxis(EAxis::X);
			if (OrbitalMotion)
			{
				// In orbital motion, project the motion vector to East/North plane
				FVector ENUMovementVector = TangentTransform.InverseTransformVector(MovementVector);
				ENUMovementVector.Z = 0;
				ENUMovementVector.Normalize();
				MovementVector = TangentTransform.TransformVector(ENUMovementVector);
			}
			// transform to world space and add it
			AddMovementInput(MovementVector, Val);

			// Allows for recovering user control
			if (bIsFlyingToLocation && bCanInterruptFlight)
			{
				InterruptFlyToLocation();
			}
		}
	}
}

void ARoundPlanetPawn::MoveRight(float Val)
{
	if (Val != 0.f)
	{
		if (GetController())
		{
			FRotator const ControlSpaceRot = this->GetViewRotation();

			FVector MovementVector = FRotationMatrix(ControlSpaceRot).GetScaledAxis(EAxis::Y);
			if (OrbitalMotion)
			{
				// In orbital motion, project the motion vector to East/North plane
				FVector ENUMovementVector = TangentTransform.InverseTransformVector(MovementVector);
				ENUMovementVector.Z = 0;
				ENUMovementVector.Normalize();
				MovementVector = TangentTransform.TransformVector(ENUMovementVector);
			}
			// transform to world space and add it
			AddMovementInput(MovementVector, Val);

			// Allows for recovering user control
			if (bIsFlyingToLocation && bCanInterruptFlight)
			{
				InterruptFlyToLocation();
			}
		}
	}
}

void ARoundPlanetPawn::MoveUp_World(float Val)
{
	if (Val != 0.f)
	{
		// Don't use UE Z here, but the Planet local Up
		AddMovementInput(Up, Val);

		if (bIsFlyingToLocation && bCanInterruptFlight)
		{
			InterruptFlyToLocation();
		}
	}
}

FRotator ARoundPlanetPawn::GetViewRotation() const
{
	// Compensate for tangent motion orientation
	FRotator LocalRotator = ADefaultPawn::GetViewRotation();
	return TangentTransform.TransformRotation(LocalRotator.Quaternion()).Rotator();
}

FRotator ARoundPlanetPawn::GetBaseAimRotation() const 
{
	return this->GetViewRotation();
}

void ARoundPlanetPawn::SetupPlayerInputComponent(UInputComponent* InInputComponent)
{
	Super::SetupPlayerInputComponent(InInputComponent);
	
	// Add MouseScroll-based speed change behavior, similarely as done in the Editor.
	check(InInputComponent);
	if (bAddDefaultMovementBindings)
	{
		static bool bRoundPlanetBindingsAdded = false;
		if (!bRoundPlanetBindingsAdded)
		{
			bRoundPlanetBindingsAdded = true;

			UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("RoundPlanetPawn_IncreaseSpeedScalar", EKeys::MouseScrollUp));
			UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("RoundPlanetPawn_DecreaseSpeedScalar", EKeys::MouseScrollDown));
			UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("RoundPlanetPawn_ResetSpeedScalar", EKeys::MiddleMouseButton));
		}

		InInputComponent->BindAction("RoundPlanetPawn_IncreaseSpeedScalar", IE_Pressed, this, &ARoundPlanetPawn::IncreaseSpeedScalar);
		InInputComponent->BindAction("RoundPlanetPawn_DecreaseSpeedScalar", IE_Pressed, this, &ARoundPlanetPawn::DecreaseSpeedScalar);
		InInputComponent->BindAction("RoundPlanetPawn_ResetSpeedScalar", IE_Pressed, this, &ARoundPlanetPawn::ResetSpeedScalar);
	}
}

// Private Methods
void ARoundPlanetPawn::ProcessFlyToLocation(float DeltaSeconds)
{
	if (bIsFlyingToLocation)
	{
		CurrentFlyTime += DeltaSeconds;
		float TimeProgress = CurrentFlyTime / FlyDuration;

		if (TimeProgress < 1.0)
		{
			// In order to accelerate at start and slow down at end, we use a progress profile curve
			float FlightProgress = TimeProgress;
			if (ProgressCurve != NULL)
			{
				FlightProgress = FMath::Clamp<float>(ProgressCurve->GetFloatValue(TimeProgress), 0.0, 0.9999);
			}

			// Find the bounding keypoint indexes corresponding to the current Flight progress
			int PreviousKeypointIndex = FMath::Floor(FlightProgress * (FlyToLocationKeypoints.Num() - 1));
			int NextKeypointIndex = PreviousKeypointIndex + 1;
			float SegmentProgress = FlightProgress * (FlyToLocationKeypoints.Num() - 1) - PreviousKeypointIndex;

			// Get the current position by interpolating between those two points
			FVector PreviousPosition = FlyToLocationKeypoints[PreviousKeypointIndex];
			FVector NextPosition = FlyToLocationKeypoints[NextKeypointIndex];
			FVector CurrentPosition = FMath::Lerp<FVector>(PreviousPosition, NextPosition, SegmentProgress);
			
			// Set Location
			SetActorLocation(CurrentPosition); 

			FTransform CurrentTangentTransform = GeoReferencingSystem->GetTangentTransformAtEngineLocation(CurrentPosition);
			FQuat OriginRotationENU = CurrentTangentTransform.InverseTransformRotation(FlyToLocationOriginRotation.Quaternion()); 
			FQuat DestinationRotationENU = CurrentTangentTransform.InverseTransformRotation(FlyToLocationDestinationRotation.Quaternion());
			
			// Interpolate rotation - ENU space, then UE space. If we don't do like that, we can get upside down. 
			FQuat CurrentRotatorENU = FMath::Lerp<FQuat>(OriginRotationENU, DestinationRotationENU, FlightProgress);
			FRotator CurrentRotator = CurrentTangentTransform.TransformRotation(CurrentRotatorENU).Rotator();
			GetController()->SetControlRotation(CurrentRotator);
		}
		else
		{
			// We reached the end - Set actual destination location 
			FVector FinalPoint = FlyToLocationKeypoints.Last();
			SetActorLocation(FinalPoint);
			GetController()->SetControlRotation(FlyToLocationDestinationRotation);
			MovementComponent->StopMovementImmediately();
			bIsFlyingToLocation = false;
			CurrentFlyTime = 0;
		}
	}
}

void ARoundPlanetPawn::HandleMotionSpeed()
{
	// Handle movement speed
	float MaxSpeed = BaseSpeedKmh * SpeedScalar * 100.0 / 3.6;  // Km/h to UE units/s
	if (SpeedByHATModifierCurve)
	{
		MaxSpeed *= SpeedByHATModifierCurve->GetFloatValue(HAT);
	}

	// We need to update the Acceleration/Deceleration accordingly and limit max speed to have a similar behavior whatever the speed scalar. 
	UFloatingPawnMovement* FPMovementComponent = Cast<UFloatingPawnMovement>(MovementComponent);
	{
		FPMovementComponent->MaxSpeed = MaxSpeed;
		FPMovementComponent->Acceleration = MaxSpeed * 4000.f / 1200.f; // Keep the coefficient proportional to the default values. 
		FPMovementComponent->Deceleration = MaxSpeed * 8000.f / 1200.f; // Keep the coefficient proportional to the default values. 
		
		if (FPMovementComponent->IsExceedingMaxSpeed(MaxSpeed))
		{
			FVector Velocity = FPMovementComponent->Velocity;
			Velocity.Normalize();
			FPMovementComponent->Velocity = Velocity * MaxSpeed * 0.999;
		}
	}
}

void ARoundPlanetPawn::UpdateMotionObjectsCache(bool ForceUpdate /*false*/)
{
	if (!GeoReferencingSystem)
	{
		return;
	}

	// Update Local ENU Transform
	if ((this->GetActorLocation() - LastCacheLocation).SquaredLength() > CacheUpdateThresholdSq ||
		ForceUpdate)
	{
		// Update ENU Vectors
		GeoReferencingSystem->GetENUVectorsAtEngineLocation(GetActorLocation(), East, North, Up);
		FGeographicCoordinates GeographicCoordinates;
		GeoReferencingSystem->EngineToGeographic(GetActorLocation(), GeographicCoordinates);

		// Update Tangent Transform
		TangentTransform = GeoReferencingSystem->GetTangentTransformAtEngineLocation(GetActorLocation());

		// Update Altitude
		Altitude = static_cast<float>(GeographicCoordinates.Altitude);

		// Schedule a new HAT update Query
		bIsHATInvalid = true; 
	}

	if (bIsHATInvalid)
	{
		// Update HAT each step if not found previously, or scheduled by the threshold exceeded. 
		FVector LineCheckStart = GetActorLocation();
		FVector LineCheckEnd = GetActorLocation() - (Altitude + 0.05 * GeoReferencingSystem->GetGeographicEllipsoidMaxRadius()) * Up; // Estimate raycast length - Altitude + 5% of ellipsoid in case of negative altitudes
		FHitResult HitResult = FHitResult();

		static const FName LineTraceSingleName(TEXT("RoundPlanetPawnLineTrace"));
		FCollisionQueryParams CollisionParams(LineTraceSingleName);
		CollisionParams.bTraceComplex = true;
		CollisionParams.AddIgnoredActor(this);

		FCollisionObjectQueryParams ObjectParams = FCollisionObjectQueryParams(ECC_WorldStatic);
		ObjectParams.AddObjectTypesToQuery(ECC_WorldDynamic);
		ObjectParams.AddObjectTypesToQuery(ECC_Pawn);
		ObjectParams.AddObjectTypesToQuery(ECC_Visibility);

		
		if (GetWorld()->LineTraceSingleByObjectType(HitResult, LineCheckStart, LineCheckEnd, ObjectParams, CollisionParams))
		{
			HAT = FVector::Dist(GetActorLocation(), HitResult.ImpactPoint) / 100.0;
			bIsHATInvalid = false;
		}
	}
}
