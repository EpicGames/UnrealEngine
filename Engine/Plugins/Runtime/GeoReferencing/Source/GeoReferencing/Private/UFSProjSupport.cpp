// Copyright Epic Games, Inc. All Rights Reserved.

#include "UFSProjSupport.h"

#include "GeoReferencingModule.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"


struct PROJ_FILE_HANDLE
{
	FString RequestedFile;
	TUniquePtr<IFileHandle> Handle;
};

PROJ_FILE_API FUFSProj::FunctionTable =
{
	1,
	&FUFSProj::Open,
	&FUFSProj::Read,
	&FUFSProj::Write,
	&FUFSProj::Seek,
	&FUFSProj::Tell,
	&FUFSProj::Close,
	&FUFSProj::Exists,
	&FUFSProj::MkDir,
	&FUFSProj::Unlink,
	&FUFSProj::Rename,
};

#ifndef TRUE
#define TRUE (1)
#endif

#ifndef FALSE
#define FALSE (0)
#endif

static const bool EnableLogging = true;

/** Open file. Return NULL if error */
PROJ_FILE_HANDLE* FUFSProj::Open(PJ_CONTEXT* ctx, const char* filename, PROJ_OPEN_ACCESS access, void* user_data)
{
	FUTF8ToTCHAR FilePath(filename);
	UE_CLOGF(EnableLogging, LogGeoReferencing, Log, "FUFSProj::open '%ls'", FilePath.Get());

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	PROJ_FILE_HANDLE* Result = new PROJ_FILE_HANDLE;
	Result->RequestedFile = FilePath.Get();
	Result->Handle.Reset(PlatformFile.OpenRead(FilePath.Get()));
	if (!Result->Handle.IsValid())
	{
		delete Result;
		Result = nullptr;
	}

	return Result;
}

/** Read sizeBytes into buffer from current position and return number of bytes read */
size_t FUFSProj::Read(PJ_CONTEXT* ctx, PROJ_FILE_HANDLE* handle, void* buffer, size_t sizeBytes, void* user_data)
{
	UE_CLOGF(EnableLogging, LogGeoReferencing, Log, "FUFSProj::read '%ls'", *handle->RequestedFile);
	return handle->Handle->Read(static_cast<uint8*>(buffer), sizeBytes);
}

/** Write sizeBytes into buffer from current position and return number of bytes written */
size_t FUFSProj::Write(PJ_CONTEXT* ctx, PROJ_FILE_HANDLE* handle, const void* buffer, size_t sizeBytes, void* user_data)
{
	UE_CLOGF(EnableLogging, LogGeoReferencing, Log, "FUFSProj::write '%ls'", *handle->RequestedFile);
	return handle->Handle->Write(static_cast<const uint8*>(buffer), sizeBytes);
}

/** Seek to offset using whence=SEEK_SET/SEEK_CUR/SEEK_END. Return TRUE in case of success */
int FUFSProj::Seek(PJ_CONTEXT* ctx, PROJ_FILE_HANDLE* handle, long long offset, int whence, void* user_data)
{
	UE_CLOGF(EnableLogging, LogGeoReferencing, Log, "FUFSProj::seek '%ls'", *handle->RequestedFile);
	switch (whence)
	{
	case SEEK_CUR:
		return handle->Handle->Seek(handle->Handle->Tell() + offset);
	case SEEK_END:
		return handle->Handle->SeekFromEnd(offset);
	default:
		return handle->Handle->Seek(offset);
	}
}

/** Return current file position */
unsigned long long FUFSProj::Tell(PJ_CONTEXT* ctx, PROJ_FILE_HANDLE* handle, void* user_data)
{
	UE_CLOGF(EnableLogging, LogGeoReferencing, Log, "FUFSProj::tell '%ls'", *handle->RequestedFile);
	return handle->Handle->Tell();
}

/** Close file */
void FUFSProj::Close(PJ_CONTEXT* ctx, PROJ_FILE_HANDLE* handle, void* user_data)
{
	UE_CLOGF(EnableLogging, LogGeoReferencing, Log, "FUFSProj::close '%ls'", *handle->RequestedFile);
	handle->Handle = nullptr;
	delete handle;
}

/** Return TRUE if a file exists */
int FUFSProj::Exists(PJ_CONTEXT* ctx, const char* filename, void* user_data)
{
	FUTF8ToTCHAR FilePath(filename);
	UE_CLOGF(EnableLogging, LogGeoReferencing, Log, "FUFSProj::exists '%ls'", FilePath.Get());

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (PlatformFile.FileExists(FilePath.Get()))
	{
		return TRUE;
	}

	return FALSE;
}

/** Return TRUE if directory exists or could be created  */
int FUFSProj::MkDir(PJ_CONTEXT* ctx, const char* filename, void* user_data)
{
	FUTF8ToTCHAR FilePath(filename);
	UE_CLOGF(EnableLogging, LogGeoReferencing, Log, "FUFSProj::mkdir '%ls'", FilePath.Get());

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (PlatformFile.DirectoryExists(FilePath.Get()) || PlatformFile.CreateDirectory(FilePath.Get()))
	{
		return TRUE;
	}

	return FALSE;
}

/** Return TRUE if file could be removed  */
int FUFSProj::Unlink(PJ_CONTEXT* ctx, const char* filename, void* user_data)
{
	FUTF8ToTCHAR FilePath(filename);
	UE_CLOGF(EnableLogging, LogGeoReferencing, Log, "FUFSProj::unlink '%ls'", FilePath.Get());

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (PlatformFile.DeleteFile(FilePath.Get()))
	{
		return TRUE;
	}

	return FALSE;
}

/** Return TRUE if file could be renamed  */
int FUFSProj::Rename(PJ_CONTEXT* ctx, const char* oldPath, const char* newPath, void* user_data)
{
	FUTF8ToTCHAR OldFilePath(oldPath);
	FUTF8ToTCHAR NewFilePath(newPath);
	UE_CLOGF(EnableLogging, LogGeoReferencing, Log, "FUFSProj::rename '%ls' -> '%ls'", OldFilePath.Get(), NewFilePath.Get());
	return FALSE;
}
