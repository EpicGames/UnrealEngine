// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GeographicCoordinates.h"
#include "Ellipsoid.generated.h"

USTRUCT(BlueprintType)
struct GEOREFERENCING_API FEllipsoid
{
	GENERATED_USTRUCT_BODY()

public:
	FEllipsoid();
	FEllipsoid(double RadiusX, double RadiusY, double RadiusZ);
	FEllipsoid(const FVector& InRadii);

	FVector Radii;
	FVector RadiiSquared;
	FVector OneOverRadii;
	FVector OneOverRadiiSquared;

	double GetMaximumRadius();
	double GetMinimumRadius();

	FVector GeodeticSurfaceNormal(const FVector& ECEFLocation) const;
	FVector GeodeticSurfaceNormal(const FGeographicCoordinates& GeographicCoordinates) const;
};
