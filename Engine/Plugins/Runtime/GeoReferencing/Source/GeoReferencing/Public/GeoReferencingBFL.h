// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeoReferencingBFL.generated.h"

/**
 * Blueprint function library to convert geospatial coordinates to text
 */
UCLASS()
class GEOREFERENCING_API UGeoReferencingBFL : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	/**
	* Converts a LargeCoordinates value to localized formatted text, in the form 'X= Y= Z='
	**/
	UFUNCTION(BlueprintPure, Category = "GeoReferencing", meta = (DisplayName = "ToFullText", AdvancedDisplay = "1", BlueprintAutocast))
	static FText ToFullText(UPARAM(ref) FVector& CartesianCoordinates, int32 IntegralDigits = 3);

	/**
	 * Converts a LargeCoordinates value to formatted text, in the form '(X, Y, Z)'
	 **/
	UFUNCTION(BlueprintPure, Category = "GeoReferencing", meta = (DisplayName = "ToCompactText", AdvancedDisplay = "1", BlueprintAutocast))
	static FText ToCompactText(UPARAM(ref) FVector& CartesianCoordinates, int32 IntegralDigits = 3);

	/**
	 * Converts a LargeCoordinates value to 3 separate text values
	 **/
	UFUNCTION(BlueprintPure, Category = "GeoReferencing", meta = (DisplayName = "ToSeparateTexts", AdvancedDisplay = "4", BlueprintAutocast))
	static void ToSeparateTexts(UPARAM(ref) FVector& CartesianCoordinates, FText& OutX, FText& OutY, FText& OutZ, int32 IntegralDigits = 3);

};
