// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "CoreMinimal.h"

GEOREFERENCING_API DECLARE_LOG_CATEGORY_EXTERN(LogGeoReferencing, Log, All);

/**
 *  FGeoReferencingModule
 */
class GEOREFERENCING_API FGeoReferencingModule : public IModuleInterface
{
public:
	FGeoReferencingModule() = default;

	virtual void StartupModule() override;
};