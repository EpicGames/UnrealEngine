// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Info.h"
#include "Logging/LogMacros.h"
#include "Templates/PimplPtr.h"

// Double precision structures
#include "GeographicCoordinates.h"
#include "Ellipsoid.h"

#include "GeoReferencingSystem.generated.h"



UENUM(BlueprintType)
enum class EPlanetShape : uint8 {
	/**
	 * The world geometry coordinates are expressed in a projected space such as a Mercator projection.
	 * In this mode, Planet curvature is not considered and you might face errors related to projection on large environments
	 */
	FlatPlanet UMETA(DisplayName = "Flat Planet"),

	/**
	 * The world geometry coordinates are expressed in a planet wide Cartesian frame,
	 * placed on a specific location or at earth, or at the planet center.
	 * You might need dynamic rebasing to avoid precision issues at large scales.
	 */
	 RoundPlanet UMETA(DisplayName = "Round Planet"),
};

/**
 * This AInfos enable you to define a correspondance between the UE origin and an actual geographic location on a planet
 * Once done it offers different functions to convert coordinates between UE and Geographic coordinates
 */
UCLASS(hidecategories = (Transform, Replication, Actor, Cooking))
class GEOREFERENCING_API AGeoReferencingSystem : public AInfo
{
	GENERATED_BODY()

public:
	AGeoReferencingSystem(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual void PostLoad() override;
	virtual void PostActorCreated() override;
	virtual void BeginDestroy() override;

	UFUNCTION(BlueprintPure, Category = "GeoReferencing", meta = (WorldContext = "WorldContextObject"))
	static AGeoReferencingSystem* GetGeoReferencingSystem(UObject* WorldContextObject);

#pragma region New Prototypes for Blueprints
	// We want to keep the same function names, but with a change in some argument types. UBT doesn't support that unless
	// we create the new functions with a K2_ prefix and use the meta/Displayname tag to keep the same name. 
	// To allow for a simple C++ usage too, we keep the C++ functions doing the actual work separately
	// Now that we have used them for the BP Nodes, we can't move the UFUNCTION declaration to the actual new C++ function. Leave them here...

	/**
	* Convert a Vector expressed in ENGINE space to the PROJECTED CRS
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Transformations", meta = (DisplayName = "Engine To Projected"))
	void K2_EngineToProjected(const FVector& EngineCoordinates, FVector& ProjectedCoordinates) { EngineToProjected(EngineCoordinates, ProjectedCoordinates); }

	/**
	* Convert a Vector expressed in PROJECTED CRS to ENGINE space
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Transformations", meta = (DisplayName = "Projected To Engine"))
	void K2_ProjectedToEngine(const FVector& ProjectedCoordinates, FVector& EngineCoordinates) { ProjectedToEngine(ProjectedCoordinates, EngineCoordinates); }

	// Engine <--> ECEF 

	/**
	* Convert a Vector expressed in ENGINE space to the ECEF CRS
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Transformations", meta = (DisplayName = "Engine To ECEF"))
	void K2_EngineToECEF(const FVector& EngineCoordinates, FVector& ECEFCoordinates) { EngineToECEF(EngineCoordinates, ECEFCoordinates); }

	/**
	* Convert a Vector expressed in ECEF CRS to ENGINE space
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Transformations", meta = (DisplayName = "ECEF To Engine"))
	void K2_ECEFToEngine(const FVector& ECEFCoordinates, FVector& EngineCoordinates) { ECEFToEngine(ECEFCoordinates, EngineCoordinates); }

	// Projected <--> Geographic

	/**
	* Convert a Coordinate expressed in PROJECTED CRS to GEOGRAPHIC CRS
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Transformations", meta = (DisplayName = "Projected To Geographic"))
	void K2_ProjectedToGeographic(const FVector& ProjectedCoordinates, FGeographicCoordinates& GeographicCoordinates) { ProjectedToGeographic(ProjectedCoordinates, GeographicCoordinates); }

	/**
	* Convert a Coordinate expressed in GEOGRAPHIC CRS to PROJECTED CRS
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Transformations", meta = (DisplayName = "Geographic To Projected"))
	void K2_GeographicToProjected(const FGeographicCoordinates& GeographicCoordinates, FVector& ProjectedCoordinates) { GeographicToProjected(GeographicCoordinates, ProjectedCoordinates); }

	// Projected <--> ECEF

	/**
	* Convert a Coordinate expressed in PROJECTED CRS to ECEF CRS
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Transformations", meta = (DisplayName = "Projected To ECEF"))
	void K2_ProjectedToECEF(const FVector& ProjectedCoordinates, FVector& ECEFCoordinates) { ProjectedToECEF(ProjectedCoordinates, ECEFCoordinates); }

	/**
	* Convert a Coordinate expressed in ECEF CRS to PROJECTED CRS
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Transformations", meta = (DisplayName = "ECEF To Projected"))
	void K2_ECEFToProjected(const FVector& ECEFCoordinates, FVector& ProjectedCoordinates) { ECEFToProjected(ECEFCoordinates, ProjectedCoordinates); }

	// Geographic <--> ECEF

	/**
	* Convert a Coordinate expressed in GEOGRAPHIC CRS to ECEF CRS
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Transformations", meta = (DisplayName = "Geographic To ECEF"))
	void K2_GeographicToECEF(const FGeographicCoordinates& GeographicCoordinates, FVector& ECEFCoordinates) { GeographicToECEF(GeographicCoordinates, ECEFCoordinates); }

	/**
	* Convert a Coordinate expressed in ECEF CRS to GEOGRAPHIC CRS
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Transformations", meta = (DisplayName = "ECEF To Geographic"))
	void K2_ECEFToGeographic(const FVector& ECEFCoordinates, FGeographicCoordinates& GeographicCoordinates) { ECEFToGeographic(ECEFCoordinates, GeographicCoordinates); }

	// ENU & Transforms

	/**
	* Get the East North Up vectors at a specific location
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|ENU", meta = (DisplayName = "Get ENU Vectors At Projected Location"))
	void K2_GetENUVectorsAtProjectedLocation(const FVector& ProjectedCoordinates, FVector& East, FVector& North, FVector& Up) { GetENUVectorsAtProjectedLocation(ProjectedCoordinates, East, North, Up); }

	/**
	* Get the East North Up vectors at a specific location
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|ENU", meta = (DisplayName = "Get ENU Vectors At ECEF Location"))
	void K2_GetENUVectorsAtECEFLocation(const FVector& ECEFCoordinates, FVector& East, FVector& North, FVector& Up) { GetENUVectorsAtECEFLocation(ECEFCoordinates, East, North, Up); }

	/**
	* Get the East North Up vectors at a specific location - Not in engine frame, but in pure ECEF Frame !
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|ENU", meta = (DisplayName = "Get ECEF ENU Vectors At ECEF Location"))
	void K2_GetECEFENUVectorsAtECEFLocation(const FVector& ECEFCoordinates, FVector& ECEFEast, FVector& ECEFNorth, FVector& ECEFUp) { GetECEFENUVectorsAtECEFLocation(ECEFCoordinates, ECEFEast, ECEFNorth, ECEFUp); }

	/**
	* Get the the transform to locate an object tangent to Ellipsoid at a specific location
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|TangentTransforms", meta = (DisplayName = "Get Tangent Transform At Projected Location"))
	FTransform K2_GetTangentTransformAtProjectedLocation(const FVector& ProjectedCoordinates) { return GetTangentTransformAtProjectedLocation(ProjectedCoordinates); }

	/**
	* Get the the transform to locate an object tangent to Ellipsoid at a specific location
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|TangentTransforms", meta = (DisplayName = "Get Tangent Transform At ECEFLocation"))
	FTransform K2_GetTangentTransformAtECEFLocation(const FVector& ECEFCoordinates) { return GetTangentTransformAtECEFLocation(ECEFCoordinates); }

#pragma endregion

#pragma region New Prototypes for C++

	// Engine <--> Projected 

	/**
	* Convert a Vector expressed in ENGINE space to the PROJECTED CRS
	*/
	void EngineToProjected(const FVector& EngineCoordinates, FVector& ProjectedCoordinates);

	/**
	* Convert a Vector expressed in PROJECTED CRS to ENGINE space
	*/
	void ProjectedToEngine(const FVector& ProjectedCoordinates, FVector& EngineCoordinates);

	// Engine <--> ECEF 

	/**
	* Convert a Vector expressed in ENGINE space to the ECEF CRS
	*/
	void EngineToECEF(const FVector& EngineCoordinates, FVector& ECEFCoordinates);

	/**
	* Convert a Vector expressed in ECEF CRS to ENGINE space
	*/
	void ECEFToEngine(const FVector& ECEFCoordinates, FVector& EngineCoordinates);

	// Engine <--> Geographic 

	/**
	* Convert a Vector expressed in ENGINE space to the GEOGRAPHIC CRS
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Transformations")
	void EngineToGeographic(const FVector& EngineCoordinates, FGeographicCoordinates& GeographicCoordinates);

	/**
	* Convert a Vector expressed in GEOGRAPHIC CRS to ENGINE space
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Transformations")
	void GeographicToEngine(const FGeographicCoordinates& GeographicCoordinates, FVector& EngineCoordinates);


	// Projected <--> Geographic

	/**
	* Convert a Coordinate expressed in PROJECTED CRS to GEOGRAPHIC CRS
	*/
	void ProjectedToGeographic(const FVector& ProjectedCoordinates, FGeographicCoordinates& GeographicCoordinates);

	/**
	* Convert a Coordinate expressed in GEOGRAPHIC CRS to PROJECTED CRS
	*/
	void GeographicToProjected(const FGeographicCoordinates& GeographicCoordinates, FVector& ProjectedCoordinates);

	// Projected <--> ECEF

	/**
	* Convert a Coordinate expressed in PROJECTED CRS to ECEF CRS
	*/
	void ProjectedToECEF(const FVector& ProjectedCoordinates, FVector& ECEFCoordinates);

	/**
	* Convert a Coordinate expressed in ECEF CRS to PROJECTED CRS
	*/
	void ECEFToProjected(const FVector& ECEFCoordinates, FVector& ProjectedCoordinates);

	// Geographic <--> ECEF

	/**
	* Convert a Coordinate expressed in GEOGRAPHIC CRS to ECEF CRS
	*/
	void GeographicToECEF(const FGeographicCoordinates& GeographicCoordinates, FVector& ECEFCoordinates);

	/**
	* Convert a Coordinate expressed in ECEF CRS to GEOGRAPHIC CRS
	*/
	void ECEFToGeographic(const FVector& ECEFCoordinates, FGeographicCoordinates& GeographicCoordinates);

	// ENU and Transforms

	/**
	* Get the East North Up vectors at a specific location
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|ENU")
	void GetENUVectorsAtEngineLocation(const FVector& EngineCoordinates, FVector& East, FVector& North, FVector& Up);

	/**
	* Get the East North Up vectors at a specific location
	*/
	void GetENUVectorsAtProjectedLocation(const FVector& ProjectedCoordinates, FVector& East, FVector& North, FVector& Up);

	/**
	* Get the East North Up vectors at a specific location
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|ENU")
	void GetENUVectorsAtGeographicLocation(const FGeographicCoordinates& GeographicCoordinates, FVector& East, FVector& North, FVector& Up);

	/**
	* Get the East North Up vectors at a specific location
	*/
	void GetENUVectorsAtECEFLocation(const FVector& ECEFCoordinates, FVector& East, FVector& North, FVector& Up);

	/**
	* Get the East North Up vectors at a specific location - Not in engine frame, but in pure ECEF Frame !
	*/
	void GetECEFENUVectorsAtECEFLocation(const FVector& ECEFCoordinates, FVector& ECEFEast, FVector& ECEFNorth, FVector& ECEFUp);

	/**
	* Get the the transform to locate an object tangent to Ellipsoid at a specific location
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|TangentTransforms")
	FTransform GetTangentTransformAtEngineLocation(const FVector& EngineCoordinates);

	/**
	* Get the the transform to locate an object tangent to Ellipsoid at a specific location
	*/
	FTransform GetTangentTransformAtProjectedLocation(const FVector& ProjectedCoordinates);

	/**
	* Get the the transform to locate an object tangent to Ellipsoid at a specific location
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|TangentTransforms")
	FTransform GetTangentTransformAtGeographicLocation(const FGeographicCoordinates& GeographicCoordinates);

	/**
	* Get the the transform to locate an object tangent to Ellipsoid at a specific location
	*/
	FTransform GetTangentTransformAtECEFLocation(const FVector& ECEFCoordinates);

	/**
	* Set this transform to an Ellipsoid to have it positioned tangent to the origin.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Misc")
	FTransform GetPlanetCenterTransform();

	// Public PROJ Utilities 

	/**
	* Check if the string corresponds to a valid CRS descriptor
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Misc")
	bool IsCRSStringValid(FString CRSString, FString& Error);


	// Ellipsoid

	/**
	* Find the underlying Geographic CRS Ellipsoid and return its maximum radius
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Ellipsoid")
	double GetGeographicEllipsoidMaxRadius();
	/**
	* Find the underlying Geographic CRS Ellipsoid and return its minimum radius
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Ellipsoid")
	double GetGeographicEllipsoidMinRadius();

	/**
	* Find the underlying Projected CRS Ellipsoid and return its maximum radius
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Ellipsoid")
	double GetProjectedEllipsoidMaxRadius();
	/**
	* Find the underlying Projected CRS Ellipsoid and return its minimum radius
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Ellipsoid")
	double GetProjectedEllipsoidMinRadius();



	/**
	* In case you want to change the Origin or CRS definition properties during application execution, you need to call this function to update the internal transformation cache.
	* Note this is not a recommended practice, because it will not move the level actors accordingly.
	* Can be useful though if you rebase your actors yourself, or if you want to change one CRS used for displaying coordinates.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Misc", meta = (DisplayName = "Apply Runtime Changes"))
	void ApplySettings();

#pragma endregion

	//////////////////////////////////////////////////////////////////////////
	// General

	/**
	* This mode has to be set consistently with the way you authored your ground geometry.
	*  - For small environments, a projection is often applied and the world is considered as Flat
	*  - For planet scale environments, projections is not suitable and the geometry is Rounded.
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "GeoReferencing")
	EPlanetShape PlanetShape = EPlanetShape::FlatPlanet;

	/**
	* String that describes the PROJECTED CRS of choice.
	*    CRS can be identified by their code (EPSG:4326), a well-known text(WKT) string, or PROJ strings...
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = "GeoReferencing")
	FString ProjectedCRS = FString(TEXT("EPSG:32631")); // UTM Zone 31 North - (0,0) https://epsg.io/32631

	/**
	* String that describes the GEOGRAPHIC CRS of choice.
	*    CRS can be identified by their code (EPSG:4326), a well-known text(WKT) string, or PROJ strings...
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = "GeoReferencing")
	FString GeographicCRS = FString(TEXT("EPSG:4326")); // WGS84 https://epsg.io/4326

	//////////////////////////////////////////////////////////////////////////
	// Origin Location

	/**
	* if true, the UE origin is located at the Planet Center, otherwise,
	* the UE origin is assuming to be defined at one specific point of
	* the planet surface, defined by the properties below.
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "GeoReferencing|Origin Location", meta = (EditConditionHides, EditCondition = "PlanetShape==EPlanetShape::RoundPlanet"))
	bool bOriginAtPlanetCenter = false;

	/**
	* if true, the UE origin georeference is expressed in the PROJECTED CRS.
	*     (NOT in ECEF ! - Projected worlds are the most frequent use case...)
	* if false, the origin is located using geographic coordinates.
	*
	* WARNING : the location has to be expressed as Integer values because of accuracy.
	* Be very careful about that when authoring your data in external tools !
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "GeoReferencing|Origin Location", meta = (EditConditionHides, EditCondition = "bOriginAtPlanetCenter==false"))
	bool bOriginLocationInProjectedCRS = true;

	/**
	* Latitude of UE Origin on planet
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "GeoReferencing|Origin Location", meta = (EditConditionHides, EditCondition = "!bOriginLocationInProjectedCRS && !bOriginAtPlanetCenter", ClampMin = "-90", ClampMax = "90"))
	double OriginLatitude = 0.0;

	/**
	* Longitude of UE Origin on planet
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "GeoReferencing|Origin Location", meta = (EditConditionHides, EditCondition = "!bOriginLocationInProjectedCRS && !bOriginAtPlanetCenter", ClampMin = "-180", ClampMax = "180"))
	double OriginLongitude = 0.0;

	/**
	* Altitude of UE Origin on planet
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "GeoReferencing|Origin Location", meta = (EditConditionHides, EditCondition = "!bOriginLocationInProjectedCRS && !bOriginAtPlanetCenter"))
	double OriginAltitude = 0.0;

	/**
	* Easting position of UE Origin on planet, express in the Projected CRS Frame
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "GeoReferencing|Origin Location", meta = (EditConditionHides, EditCondition = "bOriginLocationInProjectedCRS && !bOriginAtPlanetCenter"))
	double OriginProjectedCoordinatesEasting = 500000.0;

	/**
	* Northing position of UE Origin on planet, express in the Projected CRS Frame
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "GeoReferencing|Origin Location", meta = (EditConditionHides, EditCondition = "bOriginLocationInProjectedCRS && !bOriginAtPlanetCenter"))
	double OriginProjectedCoordinatesNorthing = 5000000.0;

	/**
	* Up position of UE Origin on planet, express in the Projected CRS Frame
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "GeoReferencing|Origin Location", meta = (EditConditionHides, EditCondition = "bOriginLocationInProjectedCRS && !bOriginAtPlanetCenter"))
	double OriginProjectedCoordinatesUp = 0.0;


#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

private:
	void Initialize();

private:
	class FGeoReferencingSystemInternals;
	TPimplPtr<FGeoReferencingSystemInternals> Impl;
};
