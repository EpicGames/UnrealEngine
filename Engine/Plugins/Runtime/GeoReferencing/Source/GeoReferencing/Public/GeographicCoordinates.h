// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeoReferencingModule.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "GeographicCoordinates.generated.h"

USTRUCT(BlueprintType)
struct GEOREFERENCING_API FGeographicCoordinates
{
	GENERATED_USTRUCT_BODY()

public:

	FGeographicCoordinates();
	FGeographicCoordinates(double InLongitude, double InLatitude, double InAltitude);
	FGeographicCoordinates(const FVector& LatLongAltVector); // FVector where X = Latitude, Y = Longitude, Z = Altitude

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = "GeoReferencing")
	double Longitude;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = "GeoReferencing")
	double Latitude;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = "GeoReferencing")
	double Altitude;

	FText ToFullText(int32 IntegralDigitsLatLon = 8, int32 IntegralDigitsAlti = 2, bool bAsDMS = false);
	FText ToCompactText(int32 IntegralDigitsLatLon = 8, int32 IntegralDigitsAlti = 2, bool bAsDMS = false);
	void ToSeparateTexts(FText& OutLatitude, FText& OutLongitude, FText& OutAltitude, int32 IntegralDigitsLatLon = 8, int32 IntegralDigitsAlti = 2, bool bAsDMS = false);

private:
	static FText AsAngle(double Val, const FNumberFormattingOptions* const Options = NULL, bool bAsDMS = false);
};

UCLASS()
class GEOREFERENCING_API UGeographicCoordinatesFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	/**
	 * Converts a GeographicCoordinates value to localized formatted text, in the form 'X= Y= Z='
	 **/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToFullText", AdvancedDisplay = "1", BlueprintAutocast), Category = "GeoReferencing")
	static inline FText ToFullText(UPARAM(ref) FGeographicCoordinates& GeographicCoordinates, int32 IntegralDigitsLatLon = 8, int32 IntegralDigitsAlti = 2, bool bAsDMS = false)
	{
		return GeographicCoordinates.ToFullText(IntegralDigitsLatLon, IntegralDigitsAlti, bAsDMS);
	}

	/**
	 * Converts a GeographicCoordinates value to formatted text, in the form '(X, Y, Z)'
	 **/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToCompactText", AdvancedDisplay = "1", BlueprintAutocast), Category = "GeoReferencing")
	static inline FText ToCompactText(UPARAM(ref) FGeographicCoordinates& GeographicCoordinates, int32 IntegralDigitsLatLon = 8, int32 IntegralDigitsAlti = 2, bool bAsDMS = false)
	{
		return GeographicCoordinates.ToCompactText(IntegralDigitsLatLon, IntegralDigitsAlti, bAsDMS);
	}

	/**
	 * Converts a GeographicCoordinates value to 3 separate text values
	 **/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToSeparateTexts", AdvancedDisplay = "4", BlueprintAutocast), Category = "GeoReferencing")
	static void ToSeparateTexts(UPARAM(ref) FGeographicCoordinates& GeographicCoordinates, FText& OutLatitude, FText& OutLongitude, FText& OutAltitude, int32 IntegralDigitsLatLon = 8, int32 IntegralDigitsAlti = 2, bool bAsDMS = false)
	{
		GeographicCoordinates.ToSeparateTexts(OutLatitude, OutLongitude, OutAltitude, IntegralDigitsLatLon, IntegralDigitsAlti, bAsDMS);
	}



	/**
	 * Make Geographic Coordinates from a FVector where X=Latitude, Y=Longitude, Z=Altitude
	 **/
	UFUNCTION(BlueprintPure, Category = "GeoReferencing|Coordinates")
	static FGeographicCoordinates MakeGeographicCoordinates(const FVector& LatLongAltVector)
	{
		return FGeographicCoordinates(LatLongAltVector);
	}

	/**
	 * Express the Geographic coordinates as a FVector where  where X=Latitude, Y=Longitude, Z=Altitude
	 **/
	UFUNCTION(BlueprintPure, Category = "GeoReferencing", meta = (DeprecatedFunction, DeprecationMessage = "BP now support doubles, Function useless and can lead to precision issues"))
	static void ToLatLongAltVector(UPARAM(ref) FGeographicCoordinates& GeographicCoordinates, FVector& OutLatLongAltVector)
	{
		OutLatLongAltVector = FVector(GeographicCoordinates.Latitude, GeographicCoordinates.Longitude, GeographicCoordinates.Altitude);
	}


};
