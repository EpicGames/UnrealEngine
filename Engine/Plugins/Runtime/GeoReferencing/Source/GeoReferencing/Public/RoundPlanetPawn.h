// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GeographicCoordinates.h"
#include "GameFramework/DefaultPawn.h"
#include "RoundPlanetPawn.generated.h"


class AGeoReferencingSystem;
class UCurveFloat;


/**
 * This pawn can be used to easily move around the globe while maintaining a
 * sensible orientation. As the pawn moves across the horizon, it automatically
 * changes its own up direction such that the world always looks right-side up.
 */
UCLASS()
class GEOREFERENCING_API ARoundPlanetPawn : public ADefaultPawn
{
	GENERATED_BODY()
	ARoundPlanetPawn();

public:

	/**
	* This curve dictates what percentage of the max altitude the pawn should take at a given time on the curve.
	* Depending on the total distance to Fly, a constant maximum altitude is computed from FlyToLocationMaximumAltitudeCurve
	* Then during the fly, the actual pawn altitude is computed from this profile and the reference maximum altitude. 
	* This allows to have nice movements whenever you FlyTo near or far from your actual location. 
	* This curve must be kept in the 0 to 1 range on both axes.
	*/
	UPROPERTY(EditAnywhere, Category = "FlyToLocation")
	TObjectPtr<UCurveFloat> AltitudeProfileCurve;

	/**
	* This curve is used to pick up a reference the maximum altitude when flying to a location. 
	* This maximum altitude will be moderated by the FlyToLocationAltitudeProfileCurve
	* X Axis is the distance of the flight (meters)
	* Y Axis is the maximum altitude to be used for this flight (meters)
	*/
	UPROPERTY(EditAnywhere, Category = "FlyToLocation")
	TObjectPtr<UCurveFloat> MaximumAltitudeCurve;

	/**
	* In case MaximumAltitudeCurve is not defined, the AltitudeProfileCurve will use this Maximum altitude value for the flight, whatever the travel distance. 
	* In meters
	*/
	UPROPERTY(EditAnywhere, Category = "FlyToLocation")
	float MaximumAltitudeValue = 300;

	/** 
	* This curve is used to ease in an out the Fly to Location speed.
	* It must be kept in the 0 to 1 range on both axes.
	* X axis is the normalized time of travel 
	* Y axis is the progress on the flight motion. 
	* A linear curve will give a constant progress speed whereas a easein//easeout one will slow down the speed at beginning/end of motion. 
	*/
	UPROPERTY(EditAnywhere, Category = "FlyToLocation")
	TObjectPtr<UCurveFloat> ProgressCurve;

	/**
	* Fly to Location duration (in seconds)
	*/
	UPROPERTY(EditAnywhere, Category = "FlyToLocation", meta = (ClampMin = 0.0))
	float FlyDuration = 5.0;

	/**
	* The granularity in degrees with which keypoints should be generated for the Fly to Location interpolation. 
	* Lower values means smoother motion. 
	* ROM : 1 degree latitude ~110km on earth. => 0.1 ~10km 
	*/
	UPROPERTY(EditAnywhere, Category = "FlyToLocation", meta = (ClampMin = 0.0))
	float GranularityDegrees = 0.1f;

	/**
	* The minimum linear steps for the FlyTolocation motion. 
	* GranularityDegrees is not sufficient in case of short travels. Eg if 0.1, any jump shorter than 10km will have only one step. 
	* Make sure we have at least this number of steps in the trajectory. 
	*/
	UPROPERTY(EditAnywhere, Category = "FlyToLocation", meta = (ClampMin = 0.0))
	int32 MinimumStepCount = 10;

	/**
	* Make sure we don't get crazy in case of large flights with small granularity
	*/
	UPROPERTY(EditAnywhere, Category = "FlyToLocation", meta = (ClampMin = 0.0))
	int32 MaximumStepCount = 300;


	/**
	* if True, the motion Forward/Right motion of the pawn are relative to Planet tangent, 
	* meaning the altitude will approximately be kept, whatever the pawn camera orientation
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movement")
	bool OrbitalMotion = false;

	/**
	* The Reference maximum speed for the pawn, before being altered by any Scalar modifier or by altitude curve
	* ActualMaxSpeed = BaseSpeedKmh * SpeedScalar * AltitudeSpeedModifierCurve(Altitude)
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movement")
	float BaseSpeedKmh = 100.0f;

	/**
	* Scalar modifier for the base speed
	* ActualMaxSpeed = BaseSpeedKmh * SpeedScalar * AltitudeSpeedModifierCurve(Altitude)
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movement")
	float SpeedScalar = 1.0f;

	/**
	* Multiplier/Divider for increasing/decreasing the speed scalar
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movement")
	float SpeedScalarIncrement = 1.5f;

	/**
	* When being very high and moving around the planet, we have to dynamically increase the speed based on Height above terrain to accelerate movement
	* This curve adds a multiplying factor to the maximum speed depending on HAT. 
	* ActualMaxSpeed = BaseSpeedKmh * SpeedScalar * AltitudeSpeedModifierCurve(Altitude)
	*/
	UPROPERTY(EditAnywhere, Category = "Movement")
	TObjectPtr<UCurveFloat> SpeedByHATModifierCurve;

	/**
	* Height Above Terrain. The distance between the ground and the pawn
	*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movement")
	float HAT = 0;

	/**
	* The distance between the geographic ellipsoid surface and the pawn
	*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Movement")
	float Altitude = 0;

public:
	/**
	* Begin a smooth camera flight to the given ECEF destination such that the camera ends at the specified yaw and pitch.
	* The flight can be enforced or canceled if the user moves the pawn
	*/
	UFUNCTION(BlueprintCallable, Category = "FlyToLocation")
	void FlyToLocationECEF(const FVector& ECEFDestination, double YawAtDestination, double PitchAtDestination, bool CanInterruptByMoving);

	/**
	* Begin a smooth camera flight to the given Latitude/Longitude destination such that the camera ends at the specified yaw and pitch.
	* The flight can be enforced or canceled if the user moves the pawn
	*/
	UFUNCTION(BlueprintCallable, Category = "FlyToLocation")
	void FlyToLocationGeographic(const FGeographicCoordinates& GeographicDestination, double YawAtDestination, double PitchAtDestination, bool CanInterruptByMoving);

	/**
	* Begin a smooth camera flight to the given Latitude/Longitude destination such that the camera ends at the specified yaw and pitch.
	* The flight can be enforced or canceled if the user moves the pawn
	*/
	UFUNCTION(BlueprintCallable, Category = "FlyToLocation")
	void FlyToLocationLatitudeLongitudeAltitude(const double& InLatitude, const double& InLongitude, const double& InAltitude, double YawAtDestination, double PitchAtDestination, bool CanInterruptByMoving);

	/**
	* Stop the current Fly To Location motion
	*/
	UFUNCTION(BlueprintCallable, Category = "FlyToLocation")
	void InterruptFlyToLocation();

	/**
	* Reset the Speed Scalar to its default value - Middle-mouse button click equivalent
	*/
	UFUNCTION(BlueprintCallable, Category = "Movement")
	void ResetSpeedScalar() { SpeedScalar = 1.0;};

	/**
	* Increase the Speed Scalar - MouseWheel Up equivalent
	*/
	UFUNCTION(BlueprintCallable, Category = "Movement")
	void IncreaseSpeedScalar() { SpeedScalar *= SpeedScalarIncrement; }

	/**
	* Decrease the Speed Scalar - MouseWheel Down equivalent
	*/
	UFUNCTION(BlueprintCallable, Category = "Movement")
	void DecreaseSpeedScalar() { SpeedScalar /= SpeedScalarIncrement; }

	// Public Overridables

	void OnConstruction(const FTransform& Transform) override;

	void BeginPlay() override;

	void Tick(float DeltaSeconds) override;

	/**
	* Input callback to move forward in local space (or backward if Val is negative).
	* @param Val Amount of movement in the forward direction (or backward if negative).
	* @see APawn::AddMovementInput()
	*/
	virtual void MoveForward(float Val) override;

	/**
	* Input callback to strafe right in local space (or left if Val is negative).
	* @param Val Amount of movement in the right direction (or left if negative).
	* @see APawn::AddMovementInput()
	*/
	virtual void MoveRight(float Val) override;

	/**
	* Input callback to move up in world space (or down if Val is negative).
	* @param Val Amount of movement in the world up direction (or down if negative).
	* @see APawn::AddMovementInput()
	*/
	virtual void MoveUp_World(float Val) override;


	/**
	* Don't return the controller rotation but a transformed rotation to consider planet roundness. 
	*/
	virtual FRotator GetViewRotation() const override;

	/**
	* Don't return the controller rotation but a transformed rotation to consider planet roundness. 
	*/
	virtual FRotator GetBaseAimRotation() const override;

	/**
	* Adds to the default pawn the Mouse Wheel inputs to tune the speed scalar
	*/
	void SetupPlayerInputComponent(UInputComponent* InInputComponent) override;

protected:
	UPROPERTY()
	TObjectPtr<AGeoReferencingSystem> GeoReferencingSystem;

private:
	/**
	 * In order to limit computations, we keep a cache for some values : TangentTransform, ENU, Altitude, HAT
	 * They are updated each time the pawn travels more than CacheUpdateThresholdSq (In UE Units). 
	 * if not HAT is found, we retry each tick. 
	*/
	void UpdateMotionObjectsCache(bool ForceUpdate = false);
	/**
	 * Updates the Movement Component Speed and Inertia based on the speed scalar changes. 
	*/
	void HandleMotionSpeed();
	/**
	* When flying to a specific location, this function computes at each tick the Planet-relative trajectory. 
	*/
	void ProcessFlyToLocation(float DeltaSeconds);
	

	// Variables for FlyToLocation
	bool bIsFlyingToLocation = false;
	bool bCanInterruptFlight = false;
	float CurrentFlyTime = 0.0;
	FRotator FlyToLocationOriginRotation;
	FRotator FlyToLocationDestinationRotation;
	TArray<FVector> FlyToLocationKeypoints;
	
	// Variables for Motion Object caches
	FVector LastCacheLocation;
	float CacheUpdateThresholdSq = 200.0 * 200.0; // 2m
	FVector East;
	FVector North;
	FVector Up;
	FTransform TangentTransform;
	bool bIsHATInvalid = true;
};
