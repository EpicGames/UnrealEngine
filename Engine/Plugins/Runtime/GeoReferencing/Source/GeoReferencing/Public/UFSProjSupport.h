// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "proj.h"

class FUFSProj
{
public:
	static PROJ_FILE_API FunctionTable;

private:
	static PROJ_FILE_HANDLE*	Open(PJ_CONTEXT* ctx, const char* filename, PROJ_OPEN_ACCESS access, void* user_data);
	static size_t				Read(PJ_CONTEXT* ctx, PROJ_FILE_HANDLE* handle, void* buffer, size_t sizeBytes, void* user_data);
	static size_t				Write(PJ_CONTEXT* ctx, PROJ_FILE_HANDLE* handle, const void* buffer, size_t sizeBytes, void* user_data);
	static int					Seek(PJ_CONTEXT* ctx, PROJ_FILE_HANDLE* handle, long long offset, int whence, void* user_data);
	static unsigned long long	Tell(PJ_CONTEXT* ctx, PROJ_FILE_HANDLE* handle, void* user_data);
	static void					Close(PJ_CONTEXT* ctx, PROJ_FILE_HANDLE* handle, void* user_data);
	static int					Exists(PJ_CONTEXT* ctx, const char* filename, void* user_data);
	static int					MkDir(PJ_CONTEXT* ctx, const char* filename, void* user_data);
	static int					Unlink(PJ_CONTEXT* ctx, const char* filename, void* user_data);
	static int					Rename(PJ_CONTEXT * ctx, const char* oldPath, const char* newPath, void* user_data);
};
