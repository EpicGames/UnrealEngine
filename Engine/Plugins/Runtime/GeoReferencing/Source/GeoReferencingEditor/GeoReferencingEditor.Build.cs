// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

using System;
using System.IO;
using System.Collections.Generic;

public class GeoReferencingEditor : ModuleRules
{
	public GeoReferencingEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"GeoReferencing",
				"InputCore",
				"Projects",
				"Slate",
				"SlateCore",
				"UnrealEd"
			}
		);
	}
}
