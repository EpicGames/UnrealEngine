// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/IPluginManager.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyle.h"

#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)


class FGeoReferencingEditorModule : public IModuleInterface
{
	TUniquePtr<FSlateStyleSet> StyleSet;


	virtual void StartupModule() override
	{
		FString RootDir = IPluginManager::Get().FindPlugin(TEXT("GeoReferencing"))->GetBaseDir() / TEXT("Resources");

		StyleSet = MakeUnique<FSlateStyleSet>(FName(TEXT("GeoReferencingStyle")));
		StyleSet->SetContentRoot(RootDir);

		StyleSet->Set("ClassIcon.GeoReferencingSystem", new IMAGE_BRUSH("GeoReferencingSystem_16x", FVector2D(16.0f, 16.0f)));
		StyleSet->Set("ClassThumbnail.GeoReferencingSystem", new IMAGE_BRUSH("GeoReferencingSystem_64x", FVector2D(64.0f, 64.0f)));

		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
	}

	virtual void ShutdownModule() override
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		StyleSet.Reset();
	}
};

IMPLEMENT_MODULE(FGeoReferencingEditorModule, GeoReferencingEditor)
