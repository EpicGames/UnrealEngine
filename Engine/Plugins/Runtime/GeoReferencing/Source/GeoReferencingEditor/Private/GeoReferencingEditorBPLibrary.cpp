// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeoReferencingEditorBPLibrary.h"

#include "Engine/HitResult.h"
#include <LevelEditorViewport.h>
#include <PhysicsEngine/PhysicsSettings.h>

void UGeoReferencingEditorBPLibrary::GetViewportCursorLocation(bool& Focused, FVector2D& Location)
{
	Focused = false;
	Location = FVector2D(-1, -1);

	if (GCurrentLevelEditingViewportClient != nullptr && GCurrentLevelEditingViewportClient->Viewport->HasFocus())
	{
		FViewportCursorLocation cursor = GCurrentLevelEditingViewportClient->GetCursorWorldLocationFromMousePos();
		FIntPoint pos = cursor.GetCursorPos();
		Location.X = pos.X;
		Location.Y = pos.Y;
		Focused = true;
	}
}

void UGeoReferencingEditorBPLibrary::GetViewportCursorInformation(bool& Focused, FVector2D& ScreenLocation, FVector& WorldLocation, FVector& WorldDirection)
{
	Focused = false;
	ScreenLocation = FVector2D(-1, -1);
	WorldLocation = FVector::ZeroVector;
	WorldDirection = FVector::ZeroVector;

	if (GCurrentLevelEditingViewportClient != nullptr && GCurrentLevelEditingViewportClient->Viewport->HasFocus())
	{
		FViewportCursorLocation cursorWL = GCurrentLevelEditingViewportClient->GetCursorWorldLocationFromMousePos();

		FIntPoint pos = cursorWL.GetCursorPos();
		ScreenLocation.X = pos.X;
		ScreenLocation.Y = pos.Y;
		WorldLocation = cursorWL.GetOrigin();
		WorldDirection = cursorWL.GetDirection();
		Focused = true;
	}
}

void UGeoReferencingEditorBPLibrary::LineTraceViewport(FVector2D& ScreenLocation, const TArray<AActor*>& ActorsToIgnore, const bool bTraceComplex, const bool bShowTrace, bool& bSuccess, FHitResult& HitResult)
{
	HitResult = FHitResult();
	bSuccess = false;
	bool bFocused;

	FVector WorldLocation;
	FVector WorldDirection;
	GetViewportCursorInformation(bFocused, ScreenLocation, WorldLocation, WorldDirection);
	
	if (bFocused)
	{
		LineTrace(WorldLocation, WorldDirection, ActorsToIgnore, bTraceComplex, bShowTrace, bSuccess, HitResult);
	}
}

void UGeoReferencingEditorBPLibrary::LineTrace(const FVector WorldLocation, const FVector WorldDirection, const TArray<AActor*>& ActorsToIgnore, const bool bTraceComplex, const bool bShowTrace, bool& bSuccess, FHitResult& HitResult)
{
	HitResult = FHitResult();
	bSuccess = false;
	if (GWorld == nullptr)
	{    
		return;
	}

	FVector LineCheckStart = WorldLocation;
	FVector LineCheckEnd = WorldLocation + WorldDirection * 1000000000; // 10 000 km

	static const FName LineTraceSingleName(TEXT("LevelEditorLineTrace"));
	if (bShowTrace)
	{
		GWorld->DebugDrawTraceTag = LineTraceSingleName;
	}
	else
	{
		GWorld->DebugDrawTraceTag = NAME_None;
	}

	FCollisionQueryParams CollisionParams(LineTraceSingleName);
	CollisionParams.bTraceComplex = bTraceComplex;
	CollisionParams.bReturnPhysicalMaterial = true;
	CollisionParams.bReturnFaceIndex = !UPhysicsSettings::Get()->bSuppressFaceRemapTable; // Ask for face index, as long as we didn't disable globally
	CollisionParams.AddIgnoredActors(ActorsToIgnore);

	FCollisionObjectQueryParams ObjectParams = FCollisionObjectQueryParams(ECC_WorldStatic);
	ObjectParams.AddObjectTypesToQuery(ECC_WorldDynamic);
	ObjectParams.AddObjectTypesToQuery(ECC_Pawn);

	if (GWorld->LineTraceSingleByObjectType(HitResult, LineCheckStart, LineCheckEnd, ObjectParams, CollisionParams))
	{
		bSuccess = true;
	}
}
