// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "GeoReferencingEditorBPLibrary.generated.h"

struct FHitResult;

UCLASS()
class GEOREFERENCINGEDITOR_API UGeoReferencingEditorBPLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	/**
	 * Retrieve the Viewport-Space position of the mouse in the Level Editor Viewport. If the Level editor not are in focus it will return false.
	 * 
	 * @param	Focused			If the Level editor not are in focus it will return false.
	 * @param	ScreenLocation	The screen location result.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Utilities")
	static void GetViewportCursorLocation(bool& Focused, FVector2D& ScreenLocation);

	/**
	 * Retrieve information about the viewport under the mouse cursor.
	 * 
	 * @param	Focused			If the Level editor not are in focus it will return false.
	 * @param	ScreenLocation	Viewport-Space position of cursor.
	 * @param	WorldLocation	Location of viewport origin (camera) in world space.
	 * @param	WorldDirection	Direction of viewport (camera) in world space.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Utilities")
	static void GetViewportCursorInformation(bool& Focused, FVector2D& ScreenLocation, FVector& WorldLocation, FVector& WorldDirection);

	/**
	 * LineTrace under mouse cursor and return various information
	 * 
	 * @param	ScreenLocation	Viewport-Space position of cursor.
	 * @param	ActorsToIgnore	Collection of actors for this trace to ignore.
	 * @param	TraceComplex	Whether we should trace against complex collision.
	 * @param	ShowTrace		Whether we should debug draw the trace.
	 * @param	Success			If the Level editor not are in focus it will return false, and same if nothing is hit.
	 * @param	HitResult		The trace hits result.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Utilities", meta = (AdvancedDisplay = "TraceComplex, ShowTrace, ActorsToIgnore"))
	static void LineTraceViewport(FVector2D& ScreenLocation, const TArray<AActor*>& ActorsToIgnore, const bool bTraceComplex, const bool bShowTrace, bool& bSuccess, FHitResult& HitResult);

	/**
	 * LineTrace at specific location/direction
	 * 
	 * @param	WorldLocation	Location of viewport origin (camera) in world space
	 * @param	WorldDirection	Direction of viewport (camera) in world space
	 * @param	TraceComplex	Whether we should trace against complex collision
	 * @param	ShowTrace		Whether we should debug draw the trace
	 * @param	Success			If the Level editor not are in focus it will return false, and same if nothing is hit.
	 * @param	HitResult		The trace hits result.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeoReferencing|Utilities", meta = (AdvancedDisplay = "TraceComplex, ShowTrace, ActorsToIgnore"))
	static void LineTrace(const FVector WorldLocation, const FVector WorldDirection, const TArray<AActor*>& ActorsToIgnore, const bool TraceComplex, const bool ShowTrace, bool& Success, FHitResult& HitResult);
};
